[CoreRedirects]

+ClassRedirects=(OldName="/Script/TimecodeSynchronizer.MediaPlayerTimeSynchronizationSource", NewName="/Script/MediaFrameworkUtilities.MediaPlayerTimeSynchronizationSource")
+ClassRedirects=(OldName="/Script/MediaFrameworkUtilities.MediaProfile", NewName="/Script/MediaProfile.MediaProfile")
+ClassRedirects=(OldName="/Script/MediaFrameworkUtilities.MediaProfilePlaybackManager", NewName="/Script/MediaProfile.MediaProfilePlaybackManager")
+ClassRedirects=(OldName="/Script/MediaFrameworkUtilities.ProxyMediaSource", NewName="/Script/MediaProfile.ProxyMediaSource")
+ClassRedirects=(OldName="/Script/MediaFrameworkUtilities.ProxyMediaOutput", NewName="/Script/MediaProfile.ProxyMediaOutput")
+ClassRedirects=(OldName="/Script/MediaFrameworkUtilities.MediaProfileSettings", NewName="/Script/MediaProfile.MediaProfileSettings")
+ClassRedirects=(OldName="/Script/MediaFrameworkUtilities.MediaProfileUserSettings", NewName="/Script/MediaProfile.MediaProfileUserSettings")