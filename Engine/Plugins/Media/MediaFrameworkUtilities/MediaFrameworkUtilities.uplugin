{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Media Framework Utilities",
	"Description": "This plugin provides utility assets and actors designed to simplify the Media Framework setup. It includes access to the the Media Profile editor.",
	"Category": "Media Players",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "MediaFrameworkUtilities",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		},
		{
			"Name": "MediaFrameworkUtilitiesEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "OpenCVLensDistortion",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		},
		{
			"Name": "MediaProfile",
			"Enabled": true
		}
	],
	"DocsURL": "",
	"IsExperimentalVersion": false
}
