// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MediaFrameworkUtilities : ModuleRules
	{
		public MediaFrameworkUtilities(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"MediaProfile",
					"OpenCVLensDistortion",
					"OpenCVHelper",
					"TimeManagement",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Media",
					"MediaAssets",
					"MediaIOCore",
					"MediaUtils",
				});

			if (Target.bBuildEditor == true)
			{
                PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"EditorFramework",
						"LevelEditor",
                        "MediaPlayerEditor",
                        "Slate",
						"SlateCore",
						"Settings",
						"UnrealEd"
					});
			}
		}
	}
}