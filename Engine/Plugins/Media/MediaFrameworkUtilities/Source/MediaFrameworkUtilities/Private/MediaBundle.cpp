// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaBundle.h"
#include "IAnalyticsProviderET.h"
#include "Materials/Material.h"

#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "Editor.h"
#include "EngineAnalytics.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAssetTools.h"
#include "UObject/ConstructorHelpers.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include "Engine/TextureRenderTarget2D.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Misc/CoreDelegates.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"

#define LOCTEXT_NAMESPACE "MediaBundle"


namespace MediaBundlePrivate
{
	/**
	 * Class made to handle closing bundles at the end of the frame in case it was requested during garbage collection
	 */
	class FMediaBundleDeferredCloser
	{
	public:

		static FMediaBundleDeferredCloser* Get()
		{
			static FMediaBundleDeferredCloser Instance;
			return &Instance;
		}

		/** Disallow Copying / Moving */
		FMediaBundleDeferredCloser(FMediaBundleDeferredCloser&& Other) = delete;
		FMediaBundleDeferredCloser& operator=(FMediaBundleDeferredCloser&& Other) = delete;
		FMediaBundleDeferredCloser(const FMediaBundleDeferredCloser& Other) = delete;
		FMediaBundleDeferredCloser& operator=(const FMediaBundleDeferredCloser& Other) = delete;

		~FMediaBundleDeferredCloser()
		{
			FCoreDelegates::OnEndFrame.RemoveAll(this);
		}

		void AddBundle(UMediaBundle* Bundle)
		{
			BundlesToClose.AddUnique(Bundle);
		}

		void RemoveBundle(UMediaBundle* Bundle)
		{
			BundlesToClose.RemoveSingleSwap(Bundle, EAllowShrinking::No);
		}

	private:

		FMediaBundleDeferredCloser()
		{
			FCoreDelegates::OnEndFrame.AddRaw(this, &FMediaBundleDeferredCloser::OnEndFrame);
		}

		void OnEndFrame()
		{
			for (UMediaBundle* Bundle : BundlesToClose)
			{
				//Our bundles need to close their player. Only do it if both are "valid" 
				if (IsValidChecked(Bundle) && !Bundle->IsUnreachable() && IsValidChecked(Bundle->GetMediaPlayer()) && !Bundle->GetMediaPlayer()->IsUnreachable())
				{
					Bundle->GetMediaPlayer()->Close();
				}
			}

			//Clear bundles. All closed or unusable at this point
			BundlesToClose.Empty();
		}

	private:

		TArray<UMediaBundle*> BundlesToClose;
	};
}


/* UMediaBundle
 *****************************************************************************/
UMediaBundle::UMediaBundle(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR && WITH_EDITORONLY_DATA
	static ConstructorHelpers::FObjectFinder<UMaterial> DefaultMaterialFinder(TEXT("/MediaFrameworkUtilities/M_BaseMediaBundle"));
	static ConstructorHelpers::FObjectFinder<UTexture> DefaultFailedTextureFinder(TEXT("/MediaFrameworkUtilities/T_VideoInputFailed"));
	static ConstructorHelpers::FClassFinder<AMediaBundleActorBase> DefaultActorClassFinder(TEXT("/MediaFrameworkUtilities/BP_MediaBundle_Plane_16-9"));

	DefaultMaterial = DefaultMaterialFinder.Object;
	DefaultFailedTexture = DefaultFailedTextureFinder.Object;
	DefaultActorClass = DefaultActorClassFinder.Class;

	PreviousWarningTime = 0.0;
#endif //WITH_EDITOR && WITH_EDITORONLY_DATA
}

bool UMediaBundle::OpenMediaSource()
{
	bool bResult = false;
	if (MediaSource && MediaPlayer)
	{
		bResult = true;

		if (FApp::CanEverRender())
		{
			// Only play once
			const EMediaState MediaState = MediaPlayer->GetPlayerFacade()->GetPlayer().IsValid() ? MediaPlayer->GetPlayerFacade()->GetPlayer()->GetControls().GetState() : EMediaState::Closed;
			if (MediaState == EMediaState::Closed || MediaState == EMediaState::Error)
			{
				bResult = MediaPlayer->OpenSource(MediaSource);
				MediaPlayer->SetLooping(bLoopMediaSource);
			}

			if (bResult)
			{
				MediaPlayer->OnMediaClosed.AddUniqueDynamic(this, &UMediaBundle::OnMediaClosed);
				MediaPlayer->OnMediaOpened.AddUniqueDynamic(this, &UMediaBundle::OnMediaOpenOpened);
				MediaPlayer->OnMediaOpenFailed.AddUniqueDynamic(this, &UMediaBundle::OnMediaOpenFailed);
				++ReferenceCount;

				//Clear our entry in the closer since someone else requested playback
				MediaBundlePrivate::FMediaBundleDeferredCloser::Get()->RemoveBundle(this);

				if (ReferenceCount == 1)
				{
					IMediaProfileManager::Get().OnMediaProfileChanged().AddUObject(this, &UMediaBundle::OnMediaProfileChanged);
				}
			}
#if WITH_EDITOR && WITH_EDITORONLY_DATA
			else if(GIsEditor)
			{
				const double TimeNow = FPlatformTime::Seconds();
				const double TimeBetweenWarningsInSeconds = 3.0f;

				if (TimeNow - PreviousWarningTime > TimeBetweenWarningsInSeconds)
				{
					FNotificationInfo NotificationInfo(LOCTEXT("MediaOpenFailedError", "The media failed to open. Check Output Log for details!"));
					NotificationInfo.ExpireDuration = 2.0f;
					FSlateNotificationManager::Get().AddNotification(NotificationInfo);
					PreviousWarningTime = TimeNow;
				}
			}
#endif // WITH_EDITOR && WITH_EDITORONLY_DATA
		}
	}
	return bResult;
}

void UMediaBundle::CloseMediaSource()
{
	--ReferenceCount;
	if (ReferenceCount == 0 && MediaPlayer)
	{
		//Closing the player creates a newobject (playlist). If we are garbage collecting, defer closing it to avoid asserting
		if (IsGarbageCollecting())
		{
			MediaBundlePrivate::FMediaBundleDeferredCloser::Get()->AddBundle(this);
		}
		else
		{
			MediaPlayer->Close();
		}

		IMediaProfileManager::Get().OnMediaProfileChanged().RemoveAll(this);
	}
}

bool UMediaBundle::IsPlaying() const
{
	bool bResult = false;
	if (MediaSource && MediaPlayer)
	{
		const EMediaState MediaState = MediaPlayer->GetPlayerFacade()->GetPlayer().IsValid() ? MediaPlayer->GetPlayerFacade()->GetPlayer()->GetControls().GetState() : EMediaState::Closed;
		bResult = MediaState == EMediaState::Playing;
	}
	return bResult;
}

void UMediaBundle::OnMediaClosed()
{
	check(MediaPlayer);

	const EMediaState MediaState = MediaPlayer->GetPlayerFacade()->GetPlayer().IsValid() ? MediaPlayer->GetPlayerFacade()->GetPlayer()->GetControls().GetState() : EMediaState::Closed;
	if (MediaState == EMediaState::Closed || MediaState == EMediaState::Error)
	{
		OnMediaStateChanged().Broadcast(false);

		if (bReopenSourceOnError && ReferenceCount > 0)
		{
			if (MediaSource && FApp::CanEverRender())
			{
				MediaPlayer->OpenSource(MediaSource);
				MediaPlayer->SetLooping(bLoopMediaSource);
			}
		}
	}
}

void UMediaBundle::OnMediaOpenOpened(FString DeviceUrl)
{
	OnMediaStateChanged().Broadcast(true);
}

void UMediaBundle::OnMediaOpenFailed(FString DeviceUrl)
{
	OnMediaStateChanged().Broadcast(false);
}

void UMediaBundle::OnMediaProfileChanged(UMediaProfile* OldMediaProfile, UMediaProfile* NewMediaProfile)
{
	if (ReferenceCount > 0 && MediaPlayer && MediaSource && FApp::CanEverRender())
	{
		MediaPlayer->OpenSource(MediaSource);
		MediaPlayer->SetLooping(bLoopMediaSource);
	}
}

void UMediaBundle::RefreshLensDisplacementMap()
{
	if (LensDisplacementMap)
	{
		CurrentLensParameters = LensParameters;

		if (FApp::CanEverRender())
		{
			UTexture2D* PreComputed = CurrentLensParameters.CreateUndistortUVDisplacementMap(FIntPoint(256, 256), 0.0f, UndistortedCameraViewInfo);

			if (PreComputed != nullptr)
			{
				UWorld* World = GetWorld();
				if (World == nullptr)
				{
#if WITH_EDITOR
					World = GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
#endif
				}

				if (World != nullptr)
				{
					FOpenCVLensDistortionParameters::DrawDisplacementMapToRenderTarget(World, LensDisplacementMap, PreComputed);
				}
			}
		}
	}
}

#if WITH_EDITOR
TArray<UPackage*> UMediaBundle::CreateInternalsEditor()
{
	TArray<UPackage*> Result;

	if (GIsEditor)
	{
		IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		const FString ParentName = GetOuter()->GetName() + "_InnerAssets/";
		FString OutAssetName;
		FString OutPackageName;

		//Create MediaPlayer
		AssetTools.CreateUniqueAssetName(*(ParentName + TEXT("/MediaP_") + GetName()), TEXT(""), OutPackageName, OutAssetName);
		MediaPlayer = Cast<UMediaPlayer>(AssetTools.CreateAsset(OutAssetName, ParentName, UMediaPlayer::StaticClass(), nullptr));
		MediaPlayer->AffectedByPIEHandling = false;
		Result.Add(MediaPlayer->GetOutermost());

		//Create MediaTexture 
		AssetTools.CreateUniqueAssetName(*(ParentName + TEXT("/T_") + GetName() + TEXT("_BC")), TEXT(""), OutPackageName, OutAssetName);
		MediaTexture = Cast<UMediaTexture>(AssetTools.CreateAsset(OutAssetName, ParentName, UMediaTexture::StaticClass(), nullptr));
		MediaTexture->NewStyleOutput = true;
		MediaTexture->SetDefaultMediaPlayer(MediaPlayer);
		MediaTexture->SetMediaPlayer(MediaPlayer);
		MediaTexture->UpdateResource();
		Result.Add(MediaTexture->GetOutermost());

		//Create LensDisplacementMap RenderTarget
		AssetTools.CreateUniqueAssetName(*(ParentName + TEXT("/RT_") + GetName() + TEXT("_LensDisplacement")), TEXT(""), OutPackageName, OutAssetName);
		LensDisplacementMap = Cast<UTextureRenderTarget2D>(AssetTools.CreateAsset(OutAssetName, ParentName, UTextureRenderTarget2D::StaticClass(), nullptr));
		LensDisplacementMap->RenderTargetFormat = RTF_RGBA16f;
		LensDisplacementMap->InitAutoFormat(256, 256);
		LensDisplacementMap->UpdateResource();
		Result.Add(LensDisplacementMap->GetOutermost());

		//Create MaterialInstanceConstant
		UMaterialInstanceConstantFactoryNew* Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
		Factory->InitialParent = DefaultMaterial;

		AssetTools.CreateUniqueAssetName(*(ParentName + TEXT("/MI_") + GetName()), TEXT(""), OutPackageName, OutAssetName);
		UMaterialInstanceConstant* NewMaterial = Cast<UMaterialInstanceConstant>(AssetTools.CreateAsset(OutAssetName, ParentName, UMaterialInstanceConstant::StaticClass(), Factory));
		NewMaterial->SetTextureParameterValueEditorOnly(FMaterialParameterInfo(MediaBundleMaterialParametersName::MediaTextureName), MediaTexture);
		NewMaterial->SetTextureParameterValueEditorOnly(FMaterialParameterInfo(MediaBundleMaterialParametersName::FailedTextureName), DefaultFailedTexture);
		NewMaterial->SetTextureParameterValueEditorOnly(FMaterialParameterInfo(MediaBundleMaterialParametersName::LensDisplacementMapTextureName), LensDisplacementMap);
		NewMaterial->PostEditChange();
		Material = NewMaterial;
		Result.Add(Material->GetOutermost());
	}

	//If we are creating a new object, set the default actor class. Duplicates won't change this.
	if (MediaBundleActorClass == nullptr)
	{
		MediaBundleActorClass = DefaultActorClass;
	}

	return Result;
}
#endif // WITH_EDITOR

void UMediaBundle::PostLoad()
{
	Super::PostLoad();

	if (LensDisplacementMap != nullptr)
	{
		//Handle DisplacementMap PostLoad on our own to avoid our texture being reset
		LensDisplacementMap->ConditionalPostLoad();

		//No need to clear render target. We will generate it right after.
		const bool bClearRenderTarget = false;
		LensDisplacementMap->UpdateResourceImmediate(bClearRenderTarget);

		RefreshLensDisplacementMap();
	}
}

void UMediaBundle::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITOR
	CreateInternalsEditor();
#endif
}

#if WITH_EDITOR
/**
 * @EventName MediaFramework.MediaSourceSelected
 * @Trigger Triggered when a media source is selected.
 * @Type Client
 * @Owner MediaIO Team
 */
void UMediaBundle::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMediaBundle, MediaSource))
	{
		if (MediaPlayer)
		{
			MediaPlayer->Close();
			if (MediaSource && ReferenceCount > 0 && FApp::CanEverRender())
			{
				MediaPlayer->OpenSource(MediaSource);
			}
		}
	}
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMediaBundle, bLoopMediaSource))
	{
		if (MediaPlayer)
		{
			MediaPlayer->SetLooping(bLoopMediaSource);
			const EMediaState MediaState = MediaPlayer->GetPlayerFacade()->GetPlayer().IsValid() ? MediaPlayer->GetPlayerFacade()->GetPlayer()->GetControls().GetState() : EMediaState::Closed;
			if (MediaState == EMediaState::Stopped && ReferenceCount > 0 && FApp::CanEverRender())
			{
				MediaPlayer->OpenSource(MediaSource);
			}
		}
	}
	else if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMediaBundle, MediaSource))
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<UObject*> Objects;
			
			for (int32 Index = 0; Index < PropertyChangedEvent.GetNumObjectsBeingEdited(); Index++)
			{
				if (const UObject* Object = PropertyChangedEvent.GetObjectBeingEdited(Index))
				{
					if (const UMediaSource* Source = PropertyChangedEvent.Property->ContainerPtrToValuePtr<UMediaSource>(Object))
					{
						TArray<FAnalyticsEventAttribute> EventAttributes;
						EventAttributes.Add(FAnalyticsEventAttribute(TEXT("MediaSourceType"), Source->GetClass()->GetName()));
						FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.MediaSourceSelected"), EventAttributes);
					}
				}
			}
		}
	}
	else if (CurrentLensParameters != LensParameters)
	{
		//Usage of internal current value to be able to regenerate displacement map
		//after undo/redo of a lens parameter
		RefreshLensDisplacementMap();
	}
}
#endif

#undef LOCTEXT_NAMESPACE
