// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaBundleActorBase.h"

#include "Components/PrimitiveComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "Logging/MessageLog.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MediaBundle.h"
#include "MediaSoundComponent.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"

#define LOCTEXT_NAMESPACE "MediaBundleActorErrorChecking"

void AMediaBundleActorBase::SetComponent(UPrimitiveComponent* InPrimitive, UMediaSoundComponent* InMediaSound)
{
	if (InPrimitive != PrimitiveCmp)
	{
		if (MediaBundle && PrimitiveCmp && PrimitiveCmp->GetMaterial(PrimitiveMaterialIndex) == Material)
		{
			PrimitiveCmp->SetMaterial(PrimitiveMaterialIndex, nullptr);
		}
		PrimitiveCmp = InPrimitive;
		if (MediaBundle && PrimitiveCmp)
		{
			PrimitiveCmp->SetMaterial(PrimitiveMaterialIndex, Material);
		}
	}

	if (InMediaSound != MediaSoundCmp)
	{
		if (MediaBundle && MediaSoundCmp && MediaSoundCmp->GetMediaPlayer() == MediaBundle->GetMediaPlayer())
		{
			SetSoundComponentMediaPlayer(nullptr);
		}
		MediaSoundCmp = InMediaSound;
		if (MediaBundle && MediaSoundCmp)
		{
			SetSoundComponentMediaPlayer(MediaBundle->GetMediaPlayer());
		}
	}
}

void AMediaBundleActorBase::SetSoundComponentMediaPlayer(UMediaPlayer* InMediaPlayer)
{
	MediaSoundCmp->SetMediaPlayer(InMediaPlayer);
#if WITH_EDITOR
	if (GIsEditor && (MediaSoundCmp->GetWorld() == nullptr || !MediaSoundCmp->GetWorld()->IsPlayInEditor()))
	{
		MediaSoundCmp->SetDefaultMediaPlayer(InMediaPlayer);
	}
#endif
}

void AMediaBundleActorBase::CreateDynamicMaterial()
{
	check(MediaBundle);
	Material = UMaterialInstanceDynamic::Create(MediaBundle->GetMaterial(), this, *(TEXT("MID_") + GetName()));

	//Set all parameters driven by this class
	if (GarbageMatteMask != nullptr)
	{
		Material->SetTextureParameterValue(MediaBundleMaterialParametersName::GarbageMatteTextureName, GarbageMatteMask);
	}

	SetIsValidMaterialParameter(MediaBundle->IsPlaying());
}

bool AMediaBundleActorBase::RequestOpenMediaSource()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return false;
	}
	if (bPlayingMedia)
	{
		return true;
	}

	bPlayingMedia = MediaBundle ? MediaBundle->OpenMediaSource() : false;
	return bPlayingMedia;
}

void AMediaBundleActorBase::RequestCloseMediaSource()
{
	if (MediaBundle && bPlayingMedia)
	{
		MediaBundle->CloseMediaSource();
		bPlayingMedia = false;
	}
}

void AMediaBundleActorBase::SetIsValidMaterialParameter(bool bIsPlaying)
{
	if (Material)
	{
		Material->SetScalarParameterValue(MediaBundleMaterialParametersName::IsValidMediaName, bIsPlaying ? 1.0f : 0.0f);
	}
}

void AMediaBundleActorBase::BeginPlay()
{
	Super::BeginPlay();

	if (bAutoPlay)
	{
		RequestOpenMediaSource();
	}
}

void AMediaBundleActorBase::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	RequestCloseMediaSource();

	Super::EndPlay(EndPlayReason);
}

void AMediaBundleActorBase::PostActorCreated()
{
	Super::PostActorCreated();

	if (!HasAnyFlags(RF_Transient))
	{
		if (MediaBundle && !MediaStateChangedHandle.IsValid())
		{
			MediaStateChangedHandle = MediaBundle->OnMediaStateChanged().AddUObject(this, &AMediaBundleActorBase::SetIsValidMaterialParameter);
		}

		if (bAutoPlay && bPlayWhileEditing)
		{
			RequestOpenMediaSource();
		}
	}
}

void AMediaBundleActorBase::PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph)
{
	Super::PostLoadSubobjects(OuterInstanceGraph);

	if (MediaBundle)
	{
		if (PrimitiveCmp)
		{
			PrimitiveCmp->SetMaterial(PrimitiveMaterialIndex, Material);
		}

		if (MediaSoundCmp)
		{
			SetSoundComponentMediaPlayer(MediaBundle->GetMediaPlayer());
		}

		if (!MediaStateChangedHandle.IsValid())
		{
			MediaStateChangedHandle = MediaBundle->OnMediaStateChanged().AddUObject(this, &AMediaBundleActorBase::SetIsValidMaterialParameter);
		}

		if (bAutoPlay && bPlayWhileEditing)
		{
			RequestOpenMediaSource();
		}
	}
}

namespace MediaBundleActorBasePrivate
{
	void MediaStateChangedRemove(FDelegateHandle& InHandle, UMediaBundle* InBundle)
	{
		if (InBundle && InHandle.IsValid())
		{
			InBundle->OnMediaStateChanged().Remove(InHandle);
			InHandle.Reset();
		}
	}
}

void AMediaBundleActorBase::Destroyed()
{
	RequestCloseMediaSource();
	MediaBundleActorBasePrivate::MediaStateChangedRemove(MediaStateChangedHandle, MediaBundle);

	Super::Destroyed();
}

void AMediaBundleActorBase::BeginDestroy()
{
	RequestCloseMediaSource();
	MediaBundleActorBasePrivate::MediaStateChangedRemove(MediaStateChangedHandle, MediaBundle);

	Super::BeginDestroy();
}

#if WITH_EDITOR
void AMediaBundleActorBase::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange)
	{
		bool bResetComponent = false;
		if (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, MediaBundle)
			|| PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, bAutoPlay)
			|| PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, bPlayWhileEditing))
		{
			bResetComponent = PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, MediaBundle);
			if (bResetComponent)
			{
				MediaBundleActorBasePrivate::MediaStateChangedRemove(MediaStateChangedHandle, MediaBundle);
			}
			RequestCloseMediaSource();
		}
		else if (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, PrimitiveCmp)
			|| PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, MediaSoundCmp))
		{
			bResetComponent = true;
		}


		if (bResetComponent && MediaBundle)
		{
			if (PrimitiveCmp && PrimitiveCmp->GetMaterial(PrimitiveMaterialIndex) == Material)
			{
				PrimitiveCmp->SetMaterial(PrimitiveMaterialIndex, nullptr);
			}
			else if (MediaSoundCmp && MediaSoundCmp->GetMediaPlayer() == MediaBundle->GetMediaPlayer())
			{
				SetSoundComponentMediaPlayer(nullptr);
			}
		}
	}
	else
	{
		//If we got a PreEditChange with no Property, we must be in undo/redo transaction
		//and PostEditChange will take care of starting the media back.
		//It is meant to avoid danglin ReferenceCount in MediaBundle.
		RequestCloseMediaSource();
	}
}

void AMediaBundleActorBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bSetComponent = false;
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, MediaBundle)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, bAutoPlay)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, bPlayWhileEditing))
	{
		if ((HasActorBegunPlay() && bAutoPlay) || (bPlayWhileEditing && bAutoPlay))
		{
			bPlayingMedia = RequestOpenMediaSource();
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, MediaBundle))
		{
			if (MediaBundle && !MediaStateChangedHandle.IsValid())
			{
				MediaStateChangedHandle = MediaBundle->OnMediaStateChanged().AddUObject(this, &AMediaBundleActorBase::SetIsValidMaterialParameter);
			}
		}

		bSetComponent = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, PrimitiveCmp)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, MediaSoundCmp))
	{
		bSetComponent = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AMediaBundleActorBase, GarbageMatteMask))
	{
		if (Material != nullptr)
		{
			if (GarbageMatteMask != nullptr)
			{
				Material->SetTextureParameterValue(MediaBundleMaterialParametersName::GarbageMatteTextureName, GarbageMatteMask);
			}
			else
			{
				//Since we only have one parameter, we can clear all of them. 
				//@todo : Add a way to clear a specific parameter
				Material->ClearParameterValues();
			}
		}
	}
	else if(PropertyName == NAME_None)
	{
		//If we got here with no property changed, let's kick a play request if we are setupped to play
		if ((HasActorBegunPlay() && bAutoPlay) || (bPlayWhileEditing && bAutoPlay))
		{
			bPlayingMedia = RequestOpenMediaSource();
		}
	}

	//Update Material if we're out of bound with the bundle i.e. : No material created or Material is different than Bundle
	if (MediaBundle)
	{
		if (Material == nullptr || Material->Parent != MediaBundle->GetMaterial())
		{
			//Cleanup component material if it was pointing to our material on the verge of being replaced.
			if (Material != nullptr && PrimitiveCmp != nullptr)
			{
				if (PrimitiveCmp->GetMaterial(PrimitiveMaterialIndex) == Material)
				{
					PrimitiveCmp->SetMaterial(PrimitiveMaterialIndex, nullptr);
				}
			}

			CreateDynamicMaterial();
			bSetComponent = true;
		}
	}
	else
	{
		Material = nullptr;
	}

	if (bSetComponent && MediaBundle)
	{
		if (PrimitiveCmp)
		{
			PrimitiveCmp->SetMaterial(PrimitiveMaterialIndex, Material);
		}
		if (MediaSoundCmp)
		{
			SetSoundComponentMediaPlayer(MediaBundle->GetMediaPlayer());
		}
	}
}

void AMediaBundleActorBase::CheckForErrors()
{
	Super::CheckForErrors();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		if (MediaBundle != nullptr)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
			Arguments.Add(TEXT("BundleName"), FText::FromString(MediaBundle->GetName()));

			if (MediaBundle->GetMaterial() == nullptr)
			{
				FMessageLog("MapCheck").Error()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_BundleMaterialNone", "{ActorName} : Bundle ({BundleName}) has an invalid Material"), Arguments)))
					->AddToken(FMapErrorToken::Create("MediaBundleMaterialNone"));
			}

			if (MediaBundle->GetMediaTexture() == nullptr)
			{
				FMessageLog("MapCheck").Error()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_BundleMediaTextureNone", "{ActorName} : Bundle ({BundleName}) has an invalid MediaTexture"), Arguments)))
					->AddToken(FMapErrorToken::Create("MediaBundleMediaTextureNone"));
			}

			if (MediaBundle->GetMediaPlayer() == nullptr)
			{
				FMessageLog("MapCheck").Error()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_BundleMediaPlayerNone", "{ActorName} : Bundle ({BundleName}) has an invalid MediaPlayer"), Arguments)))
					->AddToken(FMapErrorToken::Create("MediaBundleMediaPlayerNone"));
			}

			if (MediaBundle->GetLensDisplacementTexture() == nullptr)
			{
				FMessageLog("MapCheck").Error()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_BundleLensDisplacementMapNone", "{ActorName} : Bundle ({BundleName}) has an invalid lens displacement map"), Arguments)))
					->AddToken(FMapErrorToken::Create("MediaBundleLensDisplacementMapNone"));
			}
		}
	}
}

#endif //WITH_EDITOR


#undef LOCTEXT_NAMESPACE