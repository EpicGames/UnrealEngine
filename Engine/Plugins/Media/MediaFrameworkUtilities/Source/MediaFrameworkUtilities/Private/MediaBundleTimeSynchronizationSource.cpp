// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaBundleTimeSynchronizationSource.h"

#include "IMediaCache.h"
#include "IMediaPlayer.h"
#include "Math/RangeSet.h"
#include "MediaBundle.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "MediaFrameworkUtilitiesModule.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaTexture.h"
#include "TimeSynchronizableMediaSource.h"
#if WITH_EDITOR
#include "Widgets/SMediaImage.h"
#endif

namespace TimeSynchronizationSource
{
	UTimeSynchronizableMediaSource* GetTimeSynchronizableMediaSource(UMediaBundle* InMediaBundle)
	{
		check(InMediaBundle);

		UTimeSynchronizableMediaSource* Result = nullptr;

		if (UMediaSource* MediaSource = InMediaBundle->GetMediaSource())
		{
			Result = Cast<UTimeSynchronizableMediaSource>(MediaSource);
			if (!Result)
			{
				if (UProxyMediaSource* ProxyMediaSource = Cast<UProxyMediaSource>(MediaSource))
				{
					Result = Cast<UTimeSynchronizableMediaSource>(ProxyMediaSource->GetLeafMediaSource());
				}
			}
		}

		return Result;
	}
}

#if WITH_EDITOR
void UMediaBundleTimeSynchronizationSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMediaBundleTimeSynchronizationSource, MediaBundle))
	{
		if (bUseForSynchronization && MediaBundle && MediaBundle->GetMediaSource())
		{
			UTimeSynchronizableMediaSource* SynchronizableMediaSource = TimeSynchronizationSource::GetTimeSynchronizableMediaSource(MediaBundle);
			if (SynchronizableMediaSource == nullptr || !SynchronizableMediaSource->bUseTimeSynchronization)
			{
				// Warn the user that the MediaSource they just added isn't set for using time synchronization
				UE_LOGF(LogMediaFrameworkUtilities, Warning, "MediaSource '%ls' doesn't have the option to use time synchronization enabled.", *MediaBundle->MediaSource->GetName());
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

static FFrameTime TimeSpanToFrameTime(const FTimespan& Timespan, const FFrameRate& FrameRate)
{
	return FFrameTime::FromDecimal(Timespan.GetTotalSeconds() * FrameRate.AsDecimal()).RoundToFrame();
}

FFrameTime UMediaBundleTimeSynchronizationSource::GetNewestSampleTime() const
{
	TOptional<FTimespan> UseTimespan;

	if (MediaBundle && MediaBundle->GetMediaPlayer() && MediaBundle->GetMediaTexture())
	{
		const TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe>& Player = MediaBundle->GetMediaPlayer()->GetPlayerFacade()->GetPlayer();
		if (Player.IsValid())
		{
			//If there is a valid sample timestamp, we consider it as the next one to be used/rendered
			const FTimespan NextAvailableTime = MediaBundle->GetMediaTexture()->GetNextSampleTime();
			if (NextAvailableTime != FTimespan::MinValue())
			{
				UseTimespan = NextAvailableTime;
			}

			if (Player->GetCache().GetSampleCount(EMediaCacheState::Loaded) > 0)
			{
				TRangeSet<FTimespan> SampleTimes;
				if (Player->GetCache().QueryCacheState(EMediaCacheState::Loaded, SampleTimes))
				{
					//Fetch the maximun sample time from all ranges queried from the player's cache
					TRangeBound<FTimespan> MaxRangeBound = SampleTimes.GetMaxBound();
					if (MaxRangeBound.IsClosed())
					{
						const FTimespan MaxBound = MaxRangeBound.GetValue();
						UseTimespan = (UseTimespan.IsSet()) ? FMath::Max(MaxBound, UseTimespan.GetValue()) : MaxBound;
					}
				}
			}
		}
	}

	return UseTimespan.IsSet() ? TimeSpanToFrameTime(UseTimespan.GetValue(), GetFrameRate()) : FFrameTime();
}

FFrameTime UMediaBundleTimeSynchronizationSource::GetOldestSampleTime() const
{
	TOptional<FTimespan> UseTimespan;

	if (MediaBundle && MediaBundle->GetMediaPlayer() && MediaBundle->GetMediaTexture())
	{
		const TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe>& Player = MediaBundle->GetMediaPlayer()->GetPlayerFacade()->GetPlayer();
		if (Player.IsValid())
		{
			//If there is a valid sample timestamp, we consider it as the next one to be used/rendered
			const FTimespan NextAvailableTime = MediaBundle->GetMediaTexture()->GetNextSampleTime();
			if (NextAvailableTime != FTimespan::MinValue())
			{
				UseTimespan = NextAvailableTime;
			}

			if (Player->GetCache().GetSampleCount(EMediaCacheState::Loaded) > 0)
			{
				TRangeSet<FTimespan> SampleTimes;
				if (Player->GetCache().QueryCacheState(EMediaCacheState::Loaded, SampleTimes))
				{
					//Fetch the minimum sample time from all ranges queried from the player's cache
					TRangeBound<FTimespan> MinRangeBound = SampleTimes.GetMinBound();
					if (MinRangeBound.IsClosed())
					{
						const FTimespan MinBound = MinRangeBound.GetValue();
						UseTimespan = (UseTimespan.IsSet()) ? FMath::Min(MinBound, UseTimespan.GetValue()) : MinBound;
					}
				}
			}
		}
	}

	return UseTimespan.IsSet() ? TimeSpanToFrameTime(UseTimespan.GetValue(), GetFrameRate()) : FFrameTime();
}

FFrameRate UMediaBundleTimeSynchronizationSource::GetFrameRate() const
{
	if (IsReady())
	{
		UMediaPlayer* MediaPlayer = MediaBundle->GetMediaPlayer();
		check(MediaPlayer);
		const TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe>& Player = MediaPlayer->GetPlayerFacade()->GetPlayer();

		if (Player.IsValid())
		{
			//Save the FrameRate of the current track of the media player for future use
			const int32 SelectedTrack = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video);
			const int32 SelectedFormat = MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, SelectedTrack);
			const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(SelectedTrack, SelectedFormat);

			//Convert using 1001 for DropFrame FrameRate detection
			const uint32 Precision = 1001;
			return FFrameRate(FMath::RoundToInt(FrameRate * Precision), Precision);
		}
	}
	return FFrameRate();
}

bool UMediaBundleTimeSynchronizationSource::IsReady() const
{
	return MediaBundle && MediaBundle->GetMediaPlayer() && MediaBundle->GetMediaPlayer()->IsReady() && MediaBundle->GetMediaSource() && MediaBundle->GetMediaTexture();
}

bool UMediaBundleTimeSynchronizationSource::Open(const FTimeSynchronizationOpenData& InOpenData)
{
	OpenData = InOpenData;
	bool bResult = false;
	if (MediaBundle)
	{
		UMediaPlayer* MediaPlayer = MediaBundle->GetMediaPlayer();
		if (MediaPlayer)
		{
			UTimeSynchronizableMediaSource* SynchronizableMediaSource = TimeSynchronizationSource::GetTimeSynchronizableMediaSource(MediaBundle);
			if (bUseForSynchronization && (SynchronizableMediaSource == nullptr || !SynchronizableMediaSource->bUseTimeSynchronization))
			{
				UE_LOGF(LogMediaFrameworkUtilities, Error, "MediaBundle '%ls' doesn't support timecode synchronization", *MediaBundle->GetName());
			}
			else
			{
				bResult = MediaBundle->OpenMediaSource();
				if (!bResult)
				{
					UE_LOGF(LogMediaFrameworkUtilities, Error, "MediaBundle '%ls' could not be started.", *MediaBundle->GetName());
				}
				else
				{
					const auto& Player = MediaPlayer->GetPlayerFacade()->GetPlayer();
					if (!Player.IsValid())
					{
						UE_LOGF(LogMediaFrameworkUtilities, Error, "Player, for MediaBundle '%ls', is invalid.", *MediaBundle->GetName());
						MediaBundle->CloseMediaSource();
					}
				}
			}
		}
	}

	return bResult;
}

void UMediaBundleTimeSynchronizationSource::Start(const FTimeSynchronizationStartData& InStartData)
{
	StartData = InStartData;
	UMediaPlayer* MediaPlayer = MediaBundle ? MediaBundle->GetMediaPlayer() : nullptr;
	if (MediaPlayer)
	{
		const FFrameRate LocalFrameRate = GetFrameRate();
		const FFrameTime LocalStartFrame = FFrameRate::TransformTime(StartData->StartFrame, OpenData->SynchronizationFrameRate, LocalFrameRate);
		const FTimespan StartTimespan = FTimespan::FromSeconds(LocalFrameRate.AsSeconds(LocalStartFrame));

		// If this source is used for synchronization, then we'll try to seek to the start frame.
		if (bUseForSynchronization)
		{
			if (MediaPlayer->SupportsSeeking())
			{
				MediaPlayer->Seek(StartTimespan);
			}
		}

		// Otherwise, we'll at least try to set a delay so it sort of lines up.
		else
		{
			const FFrameTime MinimumTime = GetOldestSampleTime();

			// TODO: Verify this is the correct order. The comments on SetDelay seem confusing.
			// TODO: Maybe also do this for sync sources that don't support seeking? Need test cases.
			const FFrameTime DelayFrames = LocalStartFrame - MinimumTime;
			const double Delay = LocalFrameRate.AsSeconds(DelayFrames);

			if (Delay > 0)
			{
				MediaPlayer->SetTimeDelay(Delay);
			}
		}

		MediaPlayer->Play();
	}
}

void UMediaBundleTimeSynchronizationSource::Close()
{
	StartData.Reset();
	OpenData.Reset();
	if (MediaBundle)
	{
		MediaBundle->CloseMediaSource();
	}
}

FString UMediaBundleTimeSynchronizationSource::GetDisplayName() const
{
	return MediaBundle ? *MediaBundle->GetName() : FString();
}

#if WITH_EDITOR
TSharedRef<SWidget> UMediaBundleTimeSynchronizationSource::GetVisualWidget() const
{
	if (MediaBundle && MediaBundle->GetMediaTexture())
	{
		return SNew(SMediaImage, MediaBundle->GetMediaTexture());
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}
#endif
