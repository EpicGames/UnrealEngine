// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaFrameworkUtilitiesModule.h"

#include "IMediaProfileModule.h"
#include "Engine/Engine.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif //WITH_EDITOR


DEFINE_LOG_CATEGORY(LogMediaFrameworkUtilities);

#define LOCTEXT_NAMESPACE "MediaFrameworkUtilities"

/**
 * Implements the MediaFrameworkUtilitiesModule module.
 */
class FMediaFrameworkUtilitiesModule : public IMediaFrameworkUtilitiesModule
{
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

	virtual IMediaProfileManager& GetProfileManager() override
	{
		IMediaProfileModule* MediaProfileModule = FModuleManager::GetModulePtr<IMediaProfileModule>("MediaProfile");
		check(MediaProfileModule);

		return MediaProfileModule->GetProfileManager();
	}
};

IMPLEMENT_MODULE(FMediaFrameworkUtilitiesModule, MediaFrameworkUtilities);

#undef LOCTEXT_NAMESPACE
