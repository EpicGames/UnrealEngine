// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMediaFrameworkUtilities, Log, All);

class IMediaProfileManager;

/**
 * Implements the MediaFrameworkUtilitiesModule module.
 */
class IMediaFrameworkUtilitiesModule : public IModuleInterface
{
public:
	virtual IMediaProfileManager& GetProfileManager() = 0;
};
