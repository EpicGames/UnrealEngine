// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlayerTimeSynchronizationSource.h"

#include "IMediaCache.h"
#include "IMediaPlayer.h"
#include "Math/RangeSet.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "MediaFrameworkUtilitiesModule.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaTexture.h"
#include "TimeSynchronizableMediaSource.h"

#if WITH_EDITOR
#include "Widgets/SMediaImage.h"
#endif

namespace TimeSynchronizationSource
{
	UTimeSynchronizableMediaSource* GetTimeSynchronizableMediaSource(UMediaSource* InMediaSource)
	{
		check(InMediaSource);

		UTimeSynchronizableMediaSource* Result = nullptr;
		Result = Cast<UTimeSynchronizableMediaSource>(InMediaSource);
		if (!Result)
		{
			if (UProxyMediaSource* ProxyMediaSource = Cast<UProxyMediaSource>(InMediaSource))
			{
				Result = Cast<UTimeSynchronizableMediaSource>(ProxyMediaSource->GetLeafMediaSource());
			}
		}

		return Result;
	}

	static FFrameTime TimeSpanToFrameTime(const FTimespan& Timespan, const FFrameRate& FrameRate)
	{
		return FFrameTime::FromDecimal(Timespan.GetTotalSeconds() * FrameRate.AsDecimal()).RoundToFrame();
	}
}

#if WITH_EDITOR
void UMediaPlayerTimeSynchronizationSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMediaPlayerTimeSynchronizationSource, MediaSource))
	{
		if (bUseForSynchronization && MediaSource)
		{
			UTimeSynchronizableMediaSource* SynchronizableMediaSource = TimeSynchronizationSource::GetTimeSynchronizableMediaSource(MediaSource);
			if (SynchronizableMediaSource == nullptr || !SynchronizableMediaSource->bUseTimeSynchronization)
			{
				// Warn the user that the MediaSource they just added isn't set for using time synchronization
				UE_LOGF(LogMediaFrameworkUtilities, Warning, "MediaSource %ls doesn't have the option to use time synchronization enabled.", *MediaSource->GetName());
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

FFrameTime UMediaPlayerTimeSynchronizationSource::GetOldestSampleTime() const
{
	TOptional<FTimespan> UseTimespan;

	if (MediaTexture && MediaTexture->GetMediaPlayer())
	{
		// Ideally, the MediaTexture (or more likely, the TMediaSampleQueue) would be able to track
		// the current span of samples available. However, that's already prone to some threading issues
		// and trying to manage more data will only exacerbate that.

		// Therefore, we can only use the next available sample time.
		const FTimespan NextAvailableTime = MediaTexture->GetNextSampleTime();
		if (NextAvailableTime != FTimespan::MinValue())
		{
			UseTimespan = NextAvailableTime;
		}

		const TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe>& Player = MediaTexture->GetMediaPlayer()->GetPlayerFacade()->GetPlayer();
		if (Player.IsValid())
		{
			IMediaCache& Cache = Player->GetCache();
			if (Cache.GetSampleCount(EMediaCacheState::Loaded) > 0)
			{
				TRangeSet<FTimespan> SampleTimes;
				if (Cache.QueryCacheState(EMediaCacheState::Loaded, SampleTimes))
				{
					TRangeBound<FTimespan> MinRangeBound = SampleTimes.GetMinBound();
					if (MinRangeBound.IsClosed())
					{
						const FTimespan MinBound = MinRangeBound.GetValue();
						UseTimespan = (UseTimespan.IsSet()) ? FMath::Min(MinBound, UseTimespan.GetValue()) : MinBound;
					}
				}
			}
		}
	}

	return UseTimespan.IsSet() ? TimeSynchronizationSource::TimeSpanToFrameTime(UseTimespan.GetValue(), GetFrameRate()) : FFrameTime(0);
}

FFrameTime UMediaPlayerTimeSynchronizationSource::GetNewestSampleTime() const
{
	TOptional<FTimespan> UseTimespan;

	if (MediaTexture && MediaTexture->GetMediaPlayer())
	{
		// Ideally, the MediaTexture (or more likely, the TMediaSampleQueue) would be able to track
		// the current span of samples available. However, that's already prone to some threading issues
		// and trying to manage more data will only exacerbate that.

		// Therefore, we can only use the next available sample time.
		const FTimespan NextAvailableTime = MediaTexture->GetNextSampleTime();
		if (NextAvailableTime != FTimespan::MinValue())
		{
			UseTimespan = NextAvailableTime;
		}

		const TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe>& Player = MediaTexture->GetMediaPlayer()->GetPlayerFacade()->GetPlayer();
		if (Player.IsValid())
		{
			IMediaCache& Cache = Player->GetCache();
			if (Cache.GetSampleCount(EMediaCacheState::Loaded) > 0)
			{
				TRangeSet<FTimespan> SampleTimes;
				if (Cache.QueryCacheState(EMediaCacheState::Loaded, SampleTimes))
				{
					//Fetch the maximun sample time from all ranges queried from the player's cache
					TRangeBound<FTimespan> MaxRangeBound = SampleTimes.GetMaxBound();
					if (MaxRangeBound.IsClosed())
					{
						const FTimespan MaxBound = MaxRangeBound.GetValue();
						UseTimespan = (UseTimespan.IsSet()) ? FMath::Max(MaxBound, UseTimespan.GetValue()) : MaxBound;
					}
				}
			}
		}
	}

	return UseTimespan.IsSet() ? TimeSynchronizationSource::TimeSpanToFrameTime(UseTimespan.GetValue(), GetFrameRate()) : FFrameTime(0);
}

FFrameRate UMediaPlayerTimeSynchronizationSource::GetFrameRate() const
{
	if (IsReady())
	{
		UMediaPlayer* MediaPlayer = MediaTexture->GetMediaPlayer();
		check(MediaPlayer);
		const TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe>& Player = MediaPlayer->GetPlayerFacade()->GetPlayer();

		if (Player.IsValid())
		{
			//Save the FrameRate of the current track of the media player for future use
			const int32 SelectedTrack = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video);
			const int32 SelectedFormat = MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, SelectedTrack);
			const float FrameRate = MediaPlayer->GetVideoTrackFrameRate(SelectedTrack, SelectedFormat);

			//Convert using 1001 for DropFrame FrameRate detection
			const uint32 Precision = 1001;
			return FFrameRate(FMath::RoundToInt(FrameRate * Precision), Precision);
		}
	}
	return FFrameRate();
}

bool UMediaPlayerTimeSynchronizationSource::IsReady() const
{
	return MediaTexture && MediaTexture->GetMediaPlayer() && MediaTexture->GetMediaPlayer()->IsReady() && MediaSource;
}

bool UMediaPlayerTimeSynchronizationSource::Open(const FTimeSynchronizationOpenData& InOpenData)
{
	OpenData = InOpenData;

	bool bResult = false;
	if (MediaSource && MediaTexture)
	{
		UMediaPlayer* MediaPlayer = MediaTexture->GetMediaPlayer();
		if (MediaPlayer)
		{
			UTimeSynchronizableMediaSource* SynchronizableMediaSource = TimeSynchronizationSource::GetTimeSynchronizableMediaSource(MediaSource);
			if (bUseForSynchronization && (SynchronizableMediaSource == nullptr || !SynchronizableMediaSource->bUseTimeSynchronization))
			{
				UE_LOGF(LogMediaFrameworkUtilities, Error, "MediaSource '%ls' doesn't support timecode synchronization", *MediaSource->GetName());
			}
			else
			{
				bResult = MediaPlayer->OpenSource(MediaSource);
				if (!bResult)
				{
					UE_LOGF(LogMediaFrameworkUtilities, Error, "Player could not be started.");
				}
				else
				{
					const auto& Player = MediaPlayer->GetPlayerFacade()->GetPlayer();
					if (!Player.IsValid())
					{
						UE_LOGF(LogMediaFrameworkUtilities, Error, "Current player is invalid.");
						MediaPlayer->Close();
					}
				}
			}
		}
	}

	return bResult;
}

void UMediaPlayerTimeSynchronizationSource::Start(const FTimeSynchronizationStartData& InStartData)
{
	StartData = InStartData;

	UMediaPlayer* MediaPlayer = MediaTexture ? MediaTexture->GetMediaPlayer() : nullptr;
	if (MediaPlayer)
	{
		const FFrameRate LocalFrameRate = GetFrameRate();
		const FFrameTime LocalStartFrame = FFrameRate::TransformTime(StartData->StartFrame, OpenData->SynchronizationFrameRate, LocalFrameRate);
		const FTimespan StartTimespan = FTimespan::FromSeconds(LocalFrameRate.AsSeconds(LocalStartFrame));

		// If this source is used for synchronization, then we'll try to seek to the start frame.
		if (bUseForSynchronization)
		{
			if (MediaPlayer->SupportsSeeking())
			{
				MediaPlayer->Seek(StartTimespan);
			}
		}

		// Otherwise, we'll at least try to set a delay so it sort of lines up.
		else
		{
			const FFrameTime MinimumTime = GetOldestSampleTime();

			// TODO: Verify this is the correct order. The comments on SetDelay seem confusing.
			// TODO: Maybe also do this for sync sources that don't support seeking? Need test cases.
			const FFrameTime DelayFrames = LocalStartFrame - MinimumTime;
			const double Delay = LocalFrameRate.AsSeconds(DelayFrames);

			if (Delay > 0)
			{
				MediaPlayer->SetTimeDelay(Delay);
			}
		}

		MediaPlayer->Play();
	}
}

void UMediaPlayerTimeSynchronizationSource::Close()
{
	if (MediaSource && MediaTexture)
	{
		UMediaPlayer* MediaPlayer = MediaTexture->GetMediaPlayer();
		if (MediaPlayer)
		{
			MediaPlayer->Close();
		}
	}

	StartData.Reset();
	OpenData.Reset();
}

FString UMediaPlayerTimeSynchronizationSource::GetDisplayName() const
{
	return MediaTexture && MediaTexture->GetMediaPlayer() ? *MediaTexture->GetMediaPlayer()->GetName() : FString();
}

#if WITH_EDITOR
TSharedRef<SWidget> UMediaPlayerTimeSynchronizationSource::GetVisualWidget() const
{
	if (MediaTexture)
	{
		return SNew(SMediaImage, MediaTexture);
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}
#endif
