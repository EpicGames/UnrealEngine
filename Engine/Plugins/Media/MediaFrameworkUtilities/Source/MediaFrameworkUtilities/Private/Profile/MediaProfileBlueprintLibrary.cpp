// Copyright Epic Games, Inc. All Rights Reserved.

#include "Profile/MediaProfileBlueprintLibrary.h"

#include "Profile/IMediaProfileManager.h"


UMediaProfile* UMediaProfileBlueprintLibrary::GetMediaProfile()
{
	return IMediaProfileManager::Get().GetCurrentMediaProfile();
}


void UMediaProfileBlueprintLibrary::SetMediaProfile(UMediaProfile* MediaProfile)
{
	return IMediaProfileManager::Get().SetCurrentMediaProfile(MediaProfile);
}


TArray<UProxyMediaSource*> UMediaProfileBlueprintLibrary::GetAllMediaSourceProxy()
{
	return IMediaProfileManager::Get().GetAllMediaSourceProxy();
}


TArray<UProxyMediaOutput*> UMediaProfileBlueprintLibrary::GetAllMediaOutputProxy()
{
	return IMediaProfileManager::Get().GetAllMediaOutputProxy();
}
