// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Materials/MaterialLayersFunctions.h"
#include "OpenCVLensDistortionParameters.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"

#if WITH_EDITORONLY_DATA
#include "MediaBundleActorBase.h"
#endif

#include "MediaBundle.generated.h"

class UMaterial;
class UMaterialInterface;
class UMediaTexture;
class UMediaPlayer;
class UMediaProfile;
class UMediaSource;
class UTexture;
class UTextureRenderTarget2D;

namespace MediaBundleMaterialParametersName
{
	static const FName MediaTextureName("MediaTexture");
	static const FName FailedTextureName("FailedTexture");
	static const FName IsValidMediaName("IsValid");
	static const FName LensDisplacementMapTextureName("UVDisplacementMapTexture");
	static const FName GarbageMatteTextureName("GarbageMatteTexture");
}


/**
 * A bundle of the Media Asset necessary to play a video & audio
 */
UCLASS(BlueprintType, hidecategories=(Object))
class MEDIAFRAMEWORKUTILITIES_API UMediaBundle : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Get the material interface.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaBundle")
	UMaterialInterface* GetMaterial() { return Material; }

	/**
	 * Get the media player.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaBundle")
	UMediaPlayer* GetMediaPlayer() { return MediaPlayer; }

	/**
	 * Get the media texture.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaBundle")
	UMediaTexture* GetMediaTexture() { return MediaTexture; }

	/**
	 * Get the lens displacement Render Target.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaBundle")
	UTextureRenderTarget2D* GetLensDisplacementTexture() { return LensDisplacementMap; }

	/**
	 * Get the media source.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaBundle")
	UMediaSource* GetMediaSource() { return MediaSource; }

	/**
	 * Get the undistorted space camera view information.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaBundle|LensDistortion")
	FOpenCVCameraViewInfo GetUndistortedCameraViewInfo() { return UndistortedCameraViewInfo; }

public:

	/* MediaBundle internal MediaSource */
	UPROPERTY(EditAnywhere, Instanced, NoClear, Category="MediaBundle")
	TObjectPtr<UMediaSource> MediaSource;
	
	/** Controls MediaPlayer looping option */
	UPROPERTY(EditAnywhere, Category="MediaBundle")
	bool bLoopMediaSource;

	/** Automatically try to open the MediaSource again if an error is detected */
	UPROPERTY(EditAnywhere, Category="MediaBundle")
	bool bReopenSourceOnError;

#if WITH_EDITORONLY_DATA
	/* Class to spawn for that asset. */
	UPROPERTY(AdvancedDisplay, EditAnywhere, NoClear, Category="MediaBundle")
	TSubclassOf<AMediaBundleActorBase> MediaBundleActorClass;
#endif

protected:
	/* MediaBundle default MediaPlayer */
	UPROPERTY(Instanced)
	TObjectPtr<UMediaPlayer> MediaPlayer;

	/* MediaBundle default MediaTexture */
	UPROPERTY(Instanced)
	TObjectPtr<UMediaTexture> MediaTexture;

	/* MediaBundle default Material */
	UPROPERTY(Instanced)
	TObjectPtr<UMaterialInterface> Material;

	/* Lens parameters of the source */
	UPROPERTY(EditAnywhere, Category = "LensParameters")
	FOpenCVLensDistortionParameters LensParameters;

	/* CameraView information for the undistorted space */
	UPROPERTY(VisibleAnywhere, Category = "LensParameters")
	FOpenCVCameraViewInfo UndistortedCameraViewInfo;

	/* Current values of lens parameters to support undo/redo correctly */
	UPROPERTY(Transient, NonTransactional)
	FOpenCVLensDistortionParameters CurrentLensParameters;

	/* Destination of lens distortion result */
	UPROPERTY(Instanced)
	TObjectPtr<UTextureRenderTarget2D> LensDisplacementMap;

private:
	/* Internal reference counter of active media player */
	UPROPERTY(AdvancedDisplay, DuplicateTransient, Transient, VisibleDefaultsOnly, Category="MediaBundle", meta=(DisplayName="Debug: Reference Count"))
	int32 ReferenceCount;

#if WITH_EDITORONLY_DATA
	/* Default Material from the plugin*/
	UPROPERTY(transient)
	TObjectPtr<UMaterial> DefaultMaterial;

	/* Default Texture from the plugin*/
	UPROPERTY(transient)
	TObjectPtr<UTexture> DefaultFailedTexture;

	/* Default Actor Class from the plugin*/
	UPROPERTY(transient)
	TSubclassOf<AMediaBundleActorBase> DefaultActorClass;
#endif //WITH_EDITORDATA_ONLY

public:
	/**
	 * Play the media source. Only open the source if the reference count is 0. (ie. no one else opened it)
	 */
	bool OpenMediaSource();

	/**
	 * Close the media source. Only close the source if the reference count is 1. (ie. last one to close it)
	 */
	void CloseMediaSource();

	/**
	 * The media is playing. Some reference may have requested the media but it is possible that it is not playing because of an internal error.
	 */
	bool IsPlaying() const;

public:

	/** Delegate type for media state changed event */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMediaStateChanged, bool /*bIsPlaying*/);

	/**
	 * Media state changed event
	 * To use when the you want to be inform if the media is playing or not.
	 */
	FOnMediaStateChanged& OnMediaStateChanged() { return MediaStateChangedDelegate; }

private:

	/** Delegate for the media state changed event */
	FOnMediaStateChanged MediaStateChangedDelegate;

#if WITH_EDITORONLY_DATA
	/** Last time the warning has been displayed.*/
	double PreviousWarningTime;
#endif // WITH_EDITOR

private:

	/**
	 * Callback function to show the DefaultTexture
	 */
	UFUNCTION()
	void OnMediaClosed();
	UFUNCTION()
	void OnMediaOpenOpened(FString DeviceUrl);
	UFUNCTION()
	void OnMediaOpenFailed(FString DeviceUrl);

	/** Reopen the media when the media profile changed. */
	void OnMediaProfileChanged(UMediaProfile* OldMediaProfile, UMediaProfile* NewMediaProfile);

	/**
	 * Regenerate displacement map associated to lens parameters
	 */
	void RefreshLensDisplacementMap();
	
public:

#if WITH_EDITOR
	/**
	 * Create other assets required for a MediaBundle to work. Used for duplication and factory
	 */
	TArray<UPackage*> CreateInternalsEditor();
#endif
	
public:

	//~ UObject interface
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface
};
