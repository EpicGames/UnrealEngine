// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "MediaBundleActorBase.generated.h"

class UMaterialInstanceDynamic;
class UMediaBundle;
class UMediaPlayer;
class UMediaSoundComponent;
class UPrimitiveComponent;
class UTextureRenderTarget2D;


/**
 * A base actor that 
 */
UCLASS(abstract, Blueprintable)
class MEDIAFRAMEWORKUTILITIES_API AMediaBundleActorBase : public AActor
{
	GENERATED_BODY()

public:
	/**
	 * Get the Media Bundle.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|MediaBundle")
	UMediaBundle* GetMediaBundle() { return MediaBundle; }

	/**
	 * Play the Media Source.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaBundle")
	bool RequestOpenMediaSource();

	/**
	 * Close the Media Source.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaBundle")
	void RequestCloseMediaSource();

	/**
	 * Whether this actor requested the media to play.
	 */
	bool IsPlayRequested() const { return bPlayingMedia; }

protected:
	/**
	 * Assign the primitive to render on. Will change the material for the Media material.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaBundle")
	void SetComponent(UPrimitiveComponent* InPrimitive, UMediaSoundComponent* InMediaSound);

public:

	/** Texture containging 2D garbage matte mask */
	UPROPERTY(EditAnywhere, Category="Compositing")
	TObjectPtr<UTextureRenderTarget2D> GarbageMatteMask;
	
protected:

	/** Associated MediaBundle */
	UPROPERTY(EditAnywhere, NoClear, Category="MediaBundle")
	TObjectPtr<UMediaBundle> MediaBundle;

	/** Wheter to auto start the MediaPlayer */
	UPROPERTY(EditDefaultsOnly, Category="MediaBundle")
	bool bAutoPlay;

	/** Wheter to keep MediaPlayer playing when editing */
	UPROPERTY(EditDefaultsOnly, Category="MediaBundle", meta=(EditCondition="bAutoPlay"))
	bool bPlayWhileEditing;

	/** PrimitiveComponent on which to attach our Material */
	UPROPERTY(EditDefaultsOnly, Category="MediaBundle")
	TObjectPtr<UPrimitiveComponent> PrimitiveCmp;

	/** MediaSoundComponent associated to play sound of our MediaSource */
	UPROPERTY(EditDefaultsOnly, NoClear, Category = "MediaBundle")
	TObjectPtr<UMediaSoundComponent> MediaSoundCmp;
	
	/** Dynamic instance of the associated MediaBundle base Material */
	UPROPERTY(EditDefaultsOnly, Category="MediaBundle")
	TObjectPtr<UMaterialInstanceDynamic> Material;

	/** Index of the Material on the primitive */
	UPROPERTY(AdvancedDisplay, EditAnywhere, Category="MediaBundle")
	int32 PrimitiveMaterialIndex;
	
	/** Whether we're actually playing the media */
	bool bPlayingMedia;

private:

	/** Handle to handle to show the invalid material */
	FDelegateHandle MediaStateChangedHandle;

	/** Based on success or failure of MediaSource opening, will change parameter to update displayed texture */
	void SetIsValidMaterialParameter(bool bIsValid);

public:
	//~ Begin AActor Interface
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

#if WITH_EDITOR
	virtual void CheckForErrors() override;
#endif //WITH_EDITOR

	//~ Begin UObject Interface
	virtual void PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph);
	virtual void PostActorCreated() override;
	virtual void Destroyed() override;
	virtual void BeginDestroy() override;

#if WITH_EDITOR
public:
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	friend class UActorFactoryMediaBundle;
#endif

private:
	void SetSoundComponentMediaPlayer(UMediaPlayer* InMediaPlayer);
	void CreateDynamicMaterial();
};
