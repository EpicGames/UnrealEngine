// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TimeSynchronizationSource.h"

#include "MediaBundleTimeSynchronizationSource.generated.h"


class UMediaBundle;

/**
 * Synchronization Source using the Media Bundle
 */
UCLASS(EditInlineNew)
class MEDIAFRAMEWORKUTILITIES_API UMediaBundleTimeSynchronizationSource : public UTimeSynchronizationSource
{
	GENERATED_BODY()

public:
	/* Media bundle asset of this input*/
	UPROPERTY(EditAnywhere, Category=Player)
	TObjectPtr<UMediaBundle> MediaBundle;

public:

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	//~ Begin UTimeSynchronizationSource Interface
	virtual FFrameTime GetNewestSampleTime() const override;
	virtual FFrameTime GetOldestSampleTime() const override;
	virtual FFrameRate GetFrameRate() const override;
	virtual bool IsReady() const override;
	virtual bool Open(const FTimeSynchronizationOpenData& InOpenData) override;
	virtual void Start(const FTimeSynchronizationStartData& InStartData) override;
	virtual void Close() override;
	virtual FString GetDisplayName() const override;
#if WITH_EDITOR
	virtual TSharedRef<SWidget> GetVisualWidget() const override;
#endif
	//~ End UTimeSynchronizationSource Interface

private:

	TOptional<FTimeSynchronizationOpenData> OpenData;
	TOptional<FTimeSynchronizationStartData> StartData;
};
