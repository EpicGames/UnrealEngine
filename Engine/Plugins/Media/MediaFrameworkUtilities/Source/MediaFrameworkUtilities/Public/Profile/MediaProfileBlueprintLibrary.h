// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MediaProfileBlueprintLibrary.generated.h"

class UMediaProfile;
class UProxyMediaOutput;
class UProxyMediaSource;

UCLASS(meta=(ScriptName="MediaProfileLibrary"))
class MEDIAFRAMEWORKUTILITIES_API UMediaProfileBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "MediaProfile")
	static UMediaProfile* GetMediaProfile();

	UFUNCTION(BlueprintCallable, Category = "MediaProfile")
	static void SetMediaProfile(UMediaProfile* MediaProfile);

	UFUNCTION(BlueprintCallable, Category = "MediaProfile")
	static TArray<UProxyMediaSource*> GetAllMediaSourceProxy();

	UFUNCTION(BlueprintCallable, Category = "MediaProfile")
	static TArray<UProxyMediaOutput*> GetAllMediaOutputProxy();
};
