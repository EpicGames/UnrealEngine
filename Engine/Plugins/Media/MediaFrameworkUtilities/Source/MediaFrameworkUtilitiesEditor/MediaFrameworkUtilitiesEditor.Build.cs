// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MediaFrameworkUtilitiesEditor : ModuleRules
	{
		public MediaFrameworkUtilitiesEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"PropertyEditor"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"AssetDefinition",
					"AssetRegistry",
					"AssetTools",
					"ClassViewer",
					"CommonMenuExtensions",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"EditorFramework",
					"EditorStyle",
					"EditorWidgets",
					"Engine",
					"RenderCore",
					"InputCore",
					"Json",
					"JsonUtilities",
					"LevelEditor",
					"MainFrame",
					"MaterialEditor",
					"MediaAssets",
					"MediaFrameworkUtilities",
					"MediaIOCore",
					"MediaIOEditor",
                    "MediaPlayerEditor",
                    "MediaProfile",
                    "MediaProfileEditor",
                    "MediaUtils",
					"PlacementMode",
					"PropertyEditor",
					"SharedSettingsWidgets",
					"Slate",
					"SlateCore",
					"TimeManagement",
					"ToolMenus",
					"ToolWidgets",
					"UnrealEd",
					"WorkspaceMenuStructure",
				});
		}
	}
}
