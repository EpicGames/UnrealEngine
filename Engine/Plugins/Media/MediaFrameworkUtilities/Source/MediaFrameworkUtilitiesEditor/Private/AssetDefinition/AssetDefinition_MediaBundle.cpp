// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MediaBundle.h"

#include "MediaBundle.h"
#include "AssetEditor/MediaBundleEditorToolkit.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_MediaBundle"

FText UAssetDefinition_MediaBundle::GetAssetDisplayName() const
{
	return LOCTEXT("AssetName", "Media Bundle");
}

TSoftClassPtr<UObject> UAssetDefinition_MediaBundle::GetAssetClass() const
{
	return UMediaBundle::StaticClass();
}

FLinearColor UAssetDefinition_MediaBundle::GetAssetColor() const
{
	return FColor(140, 62, 35);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MediaBundle::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, LOCTEXT("CategorySection", "Other"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_MediaBundle::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (UObject* Obj : Objects)
	{
		if (UMediaBundle* Asset = Cast<UMediaBundle>(Obj))
		{
			FMediaBundleEditorToolkit::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, Asset);
		}
	}
	
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
