// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MediaProfile.h"

#include "AssetEditor/MediaProfileEditor.h"
#include "Profile/MediaProfile.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_MediaProfile"

FText UAssetDefinition_MediaProfile::GetAssetDisplayName() const
{
	return LOCTEXT("MediaProfile_AssetName", "Media Profile");
}

TSoftClassPtr<UObject> UAssetDefinition_MediaProfile::GetAssetClass() const
{
	return UMediaProfile::StaticClass();
}

FLinearColor UAssetDefinition_MediaProfile::GetAssetColor() const
{
	return FColor(140, 62, 35);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MediaProfile::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Media };
	return Categories;
}

EAssetCommandResult UAssetDefinition_MediaProfile::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (UObject* Obj : Objects)
	{
		if (UMediaProfile* Asset = Cast<UMediaProfile>(Obj))
		{
			FMediaProfileEditor::CreateMediaProfileEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, Asset);
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
