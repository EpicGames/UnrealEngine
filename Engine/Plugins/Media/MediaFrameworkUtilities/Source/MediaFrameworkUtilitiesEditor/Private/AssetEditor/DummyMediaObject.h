// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MediaOutput.h"
#include "MediaSource.h"

#include "DummyMediaObject.generated.h"

/** Placeholder to use in the details panel when the media profile has a null media source in its list of media sources */
UCLASS(HideCategories=("Media|MediaSource"))
class UDummyMediaSource : public UMediaSource
{
	GENERATED_BODY()

public:
	virtual FString GetUrl() const override { return TEXT(""); }
	virtual bool Validate() const override { return false; }
	
public:
	/** Index of the media source in the media profile's media sources list that this dummy represents */
	int32 MediaProfileIndex = INDEX_NONE;
};

/** Placeholder to use in the details panel when the media profile has a null media output in its list of media outputs */
UCLASS(HideCategories=("Output", "Media|Output"))
class UDummyMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

public:
	virtual FIntPoint GetRequestedSize() const override { return FIntPoint::ZeroValue; }
	virtual EPixelFormat GetRequestedPixelFormat() const override { return EPixelFormat::PF_Unknown; }
	
public:
	/** Index of the media output in the media profile's media outputs list that this dummy represents */
	int32 MediaProfileIndex = INDEX_NONE;
};