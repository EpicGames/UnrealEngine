// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditor/MediaBundleEditorToolkit.h"

#include "DetailsViewArgs.h"
#include "MediaBundle.h"

#include "Factories/Factory.h"
#include "IDetailsView.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MaterialEditorModule.h"
#include "Materials/MaterialInstance.h"
#include "PropertyEditorModule.h"
#include "Subsystems/ImportSubsystem.h"
#include "UI/MediaFrameworkUtilitiesEditorStyle.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/Docking/SDockTab.h"
 

#define LOCTEXT_NAMESPACE "MediaBundleEditor"

namespace MediaBundleEditorToolkit
{
	const FName AppIdentifier = TEXT("MediaBundleEditorApp");
	const FName PropertiesTabId(TEXT("MediaBundleEditor_Properties"));
	const FName Layout(TEXT("Standalone_MediaBundleEditor_Layout_v1"));
}

TSharedRef<FMediaBundleEditorToolkit> FMediaBundleEditorToolkit::CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UMediaBundle* InMediaBundle)
{
	TSharedRef<FMediaBundleEditorToolkit> NewEditor(new FMediaBundleEditorToolkit());
	NewEditor->InitMediaBundleEditor(Mode, InitToolkitHost, InMediaBundle);
	return NewEditor;
}

void FMediaBundleEditorToolkit::InitMediaBundleEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UMediaBundle* InMediaBundle)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddRaw(this, &FMediaBundleEditorToolkit::HandleAssetPostImport);

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout(MediaBundleEditorToolkit::Layout)
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.5f)
					->Split
					(
						FTabManager::NewSplitter()
						->SetSizeCoefficient(0.4f)
						->Split
						(
							FTabManager::NewStack()
							->AddTab(MediaBundleEditorToolkit::PropertiesTabId, ETabState::OpenedTab)
						)
					)
				)
			)
		);


	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	Super::InitAssetEditor(Mode, InitToolkitHost, MediaBundleEditorToolkit::AppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InMediaBundle);

	ExtendToolBar();

	// Get the list of objects to edit the details of
	TArray<UObject*> ObjectsToEditInDetailsView;
	ObjectsToEditInDetailsView.Add(InMediaBundle);

	// Ensure all objects are transactable for undo/redo in the details panel
	for (UObject* ObjectToEditInDetailsView : ObjectsToEditInDetailsView)
	{
		ObjectToEditInDetailsView->SetFlags(RF_Transactional);
	}

	if (DetailsView.IsValid())
	{
		// Make sure details window is pointing to our object
		DetailsView->SetObjects(ObjectsToEditInDetailsView);
	}
}

FMediaBundleEditorToolkit::~FMediaBundleEditorToolkit()
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
}

FName FMediaBundleEditorToolkit::GetToolkitFName() const
{
	return MediaBundleEditorToolkit::AppIdentifier;
}

FText FMediaBundleEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Media Bundle Editor");
}

FText FMediaBundleEditorToolkit::GetToolkitName() const
{
	const UObject* EditingObject = GetEditingObject();

	const bool bDirtyState = EditingObject->GetOutermost()->IsDirty();

	FFormatNamedArguments Args;
	Args.Add(TEXT("ToolkitName"), GetBaseToolkitName());
	Args.Add(TEXT("ObjectName"), FText::FromString(EditingObject->GetName()));
	Args.Add(TEXT("DirtyState"), bDirtyState ? FText::FromString(TEXT("*")) : FText::GetEmpty());
	return FText::Format(LOCTEXT("ToolkitTitle", "{ObjectName}{DirtyState} - {ToolkitName}"), Args);
}

FString FMediaBundleEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "MediaBundle ").ToString();
}

FLinearColor FMediaBundleEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.2f, 0.5f);
}

void FMediaBundleEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MediaBundleEditor", "MediaBundle Editor"));

	Super::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(MediaBundleEditorToolkit::PropertiesTabId, FOnSpawnTab::CreateSP(this, &FMediaBundleEditorToolkit::SpawnPropertiesTab))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FMediaBundleEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	Super::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(MediaBundleEditorToolkit::PropertiesTabId);
}

UMediaBundle* FMediaBundleEditorToolkit::GetMediaBundle() const
{
	return Cast<UMediaBundle>(GetEditingObject());
}

TSharedRef<SDockTab> FMediaBundleEditorToolkit::SpawnPropertiesTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == MediaBundleEditorToolkit::PropertiesTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("GenericDetailsTitle", "Details"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 0.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				DetailsView.ToSharedRef()
			]
		];
}

void FMediaBundleEditorToolkit::HandleAssetPostImport(UFactory* InFactory, UObject* InObject)
{
	if (GetEditingObject() == InObject)
	{
		// The details panel likely needs to be refreshed if an asset was imported again
		TArray<UObject*> PostImportedEditingObjects;
		PostImportedEditingObjects.Add(InObject);
		DetailsView->SetObjects(PostImportedEditingObjects);
	}
}

void FMediaBundleEditorToolkit::ExtendToolBar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([&](FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("MediaBundle Material");
			{
				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateLambda([&]()
						{
							if (UMediaBundle* Asset = GetMediaBundle())
							{
								IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>("MaterialEditor");

								UMaterialInterface* MaterialInterface = Asset->GetMaterial();
								IAssetEditorInstance* EditorInstance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(MaterialInterface, true);
								if (EditorInstance == nullptr)
								{
									UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
									if (MaterialInstance)
									{
										MaterialEditorModule->CreateMaterialInstanceEditor(EToolkitMode::Standalone, GetToolkitHost(), MaterialInstance);
									}
								}
							}
						})
					),
					NAME_None,
					LOCTEXT("MaterialEditor", "Open Material Editor"),
					LOCTEXT("Material_ToolTip", "Open Material Editor for this Media Bundle material."),
					FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), "MaterialEditor")
					);
			}
			ToolbarBuilder.EndSection();
		})
	);
	AddToolbarExtender(ToolbarExtender);

	RegenerateMenusAndToolbars();
}

void FMediaBundleEditorToolkit::RemoveEditingObject(UObject* Object)
{
	Super::RemoveEditingObject(Object);
}

#undef LOCTEXT_NAMESPACE
