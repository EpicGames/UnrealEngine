// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Toolkits/AssetEditorToolkit.h"


class FSpawnTabArgs;
class UMediaBundle;



/** Viewer/editor for a MediaBundle */
class FMediaBundleEditorToolkit : public FAssetEditorToolkit
{
private:
	using Super = FAssetEditorToolkit;

public:
	static TSharedRef<FMediaBundleEditorToolkit> CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UMediaBundle* InMediaBundle);

	/**
	 * Edits the specified table
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InMediaBundle			The MediaBundle asset to edit
	 */
	void InitMediaBundleEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UMediaBundle* InMediaBundle);
	~FMediaBundleEditorToolkit();

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void RemoveEditingObject(UObject* Object) override;

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Get the MediaBundle Asset being edited */
	class UMediaBundle* GetMediaBundle() const;

private:
	void HandleAssetPostImport(UFactory* InFactory, UObject* InObject);
	TSharedRef<SDockTab> SpawnPropertiesTab(const FSpawnTabArgs& Args);

	void ExtendToolBar();

private:
	/** Details view */
	TSharedPtr<class IDetailsView> DetailsView;
};
