// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaProfileCommands.h"




#define LOCTEXT_NAMESPACE "MediaProfileCommands"

void FMediaProfileCommands::RegisterCommands()
{
	UI_COMMAND(Apply, "Apply", "Apply changes to the media profile.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Edit, "Edit", "Edit the current media profile.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ClearCurrentMediaProfile, "Clear Current Media Profile", "Removes this media profile as the current editor media profile", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SaveLayout, "Save Layout", "Save the current layout", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SaveLayoutAs, "Save Layout As", "Saves the current layout as...", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveAllLayouts, "Remove All Layouts", "Removes all the user created layouts", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFullscreen, "Enable Fullscreen", "Enables fullscreen mode for this window", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::F11));
	UI_COMMAND(Immersive, "Immersive View", "Toggle immersive view on the viewport", EUserInterfaceActionType::Button, FInputChord(EKeys::F11));
}

#undef LOCTEXT_NAMESPACE
