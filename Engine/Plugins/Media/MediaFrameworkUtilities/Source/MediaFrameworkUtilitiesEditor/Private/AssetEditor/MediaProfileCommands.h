// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "UI/MediaFrameworkUtilitiesEditorStyle.h"


class FUICommandInfo;


class FMediaProfileCommands : public TCommands<FMediaProfileCommands>
{
public:

	FMediaProfileCommands()
		: TCommands<FMediaProfileCommands>(TEXT("ToolbarIcon"), NSLOCTEXT("Contexts", "ToolbarIcon", "Media Profile"), NAME_None, FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName())
	{}
	
	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

	/** Applies changes to the original Media Profile. */
	TSharedPtr<FUICommandInfo> Apply;

	/** Edit the current Media Profile. */
	TSharedPtr<FUICommandInfo> Edit;
	
	/** Clears out the current media profile  */
	TSharedPtr<FUICommandInfo> ClearCurrentMediaProfile;
	
	/** Saves the current media profile editor layout */
	TSharedPtr<FUICommandInfo> SaveLayout;
	
	/** Saves the current media profile editor layout */
	TSharedPtr<FUICommandInfo> SaveLayoutAs;
	
	/** Removes all media profile editor layouts */
	TSharedPtr<FUICommandInfo> RemoveAllLayouts;
	
	/** Toggles fullscreen mode on the media profile editor window */
	TSharedPtr<FUICommandInfo> ToggleFullscreen;
	
	/** Makes the media profile editor's viewport immersive */
	TSharedPtr<FUICommandInfo> Immersive;
};
