// Copyright Epic Games, Inc. All Rights Reserved.


#include "MediaProfileEditor.h"

#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "Engine/TextureRenderTarget2D.h"
#include "LevelEditorViewport.h"
#include "MediaProfileEditorUserSettings.h"
#include "SMediaProfileDetailsPanel.h"
#include "SMediaProfileSourcesTreeView.h"
#include "SMediaProfileViewport.h"
#include "CaptureCardMediaSource.h"
#include "MediaProfileCommands.h"
#include "SPrimaryButton.h"
#include "CaptureTab/SMediaFrameworkCapture.h"
#include "Dom/JsonObject.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "StatusBarSubsystem.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "Profile/MediaProfileSettings.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Slate/SceneViewport.h"
#include "UI/MediaFrameworkUtilitiesEditorStyle.h"
#include "UI/SMediaFrameworkTimecodeGenlockHeader.h"
#include "UI/SMediaFrameworkTimecodeGenlockPanel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Misc/ConfigCacheIni.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "MediaProfileEditor"

const FName FMediaProfileEditor::AppName =  FName(TEXT("MediaProfileEditor"));
const FName FMediaProfileEditor::MediaOutputTabId =  FName(TEXT("MediaOutput"));
const FName FMediaProfileEditor::MediaTreeTabId =  FName(TEXT("MediaTree"));
const FName FMediaProfileEditor::DetailsTabId =  FName(TEXT("Details"));
const FName FMediaProfileEditor::TimecodeTabId =  FName(TEXT("Timecode"));
const FString FMediaProfileEditor::DefaultLayoutName = TEXT("Default Layout");

namespace UE::MediaProfileEditor::Private
{
	class SSaveLayoutAsDialog : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSaveLayoutAsDialog) {}
		SLATE_END_ARGS()

		virtual void Construct(const FArguments& InArgs)
		{
			ChildSlot
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				.Padding(16.0f)
				[
					SNew(SBox)
					.WidthOverride(300.0f)
					[
						// Layout Name
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SaveLayoutAsLayoutNameLabel", "Name"))
								.Margin(FMargin(0, 0, 8, 8))
							]
							+SHorizontalBox::Slot()
							.Padding(0, 0, 0, 8)
							[
								SNew(SEditableTextBox)
								.Text_Lambda([this]() { return FText::FromString(EnteredLayoutName); })
								.OnTextCommitted(this, &SSaveLayoutAsDialog::CommitLayoutName)
								.OnTextChanged(this, &SSaveLayoutAsDialog::SetLayoutName)
								.SelectAllTextWhenFocused(true)
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							// Constant height, whether the label is visible or not
							SNew(SBox)
							.HeightOverride(20.f)
							[
								SNew(SBorder)
								.Visibility_Lambda([this] { return bHasError ? EVisibility::Visible : EVisibility::Hidden; })
								.BorderImage( FAppStyle::GetBrush("AssetDialog.ErrorLabelBorder") )
								.Content()
								[
									SNew(STextBlock)
									.Text_Lambda([this] { return ErrorText; })
								]
							]
						]
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Bottom)
							.Padding(0,0,8,0)
							[
								SNew(SPrimaryButton)
								.Text(LOCTEXT("SaveLayoutAsDialogSaveButton", "Save"))
								.IsEnabled(this, &SSaveLayoutAsDialog::IsConfirmButtonEnabled)
								.OnClicked(this, &SSaveLayoutAsDialog::OnConfirmButtonClicked)
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Bottom)
							[
								SNew(SButton)
								.Text(LOCTEXT("SaveLayoutAsDialogCancelButton", "Cancel"))
								.OnClicked(this, &SSaveLayoutAsDialog::OnCancelButtonClicked)
							]
						]
					]
				]
			];
		}
		
		FString GetLayoutName() const
		{
			if (bConfirmed)
			{
				return EnteredLayoutName;
			}
			
			return TEXT("");
		}
		
	private:
		void CloseDialog()
		{
			TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

			if (ContainingWindow.IsValid())
			{
				ContainingWindow->RequestDestroyWindow();
			}
		}
		
		void SetLayoutName(const FText& InNewName)
		{
			bHasError = false;
			ErrorText = FText::GetEmpty();
			
			EnteredLayoutName = InNewName.ToString();
			
			bool bIsUniqueName = true;
			if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
			{
				bIsUniqueName = !UserSettings->SavedLayouts.ContainsByPredicate([Name = EnteredLayoutName](const FMediaProfileEditorLayoutSettings& LayoutSettings)
				{
					return LayoutSettings.Name == Name;
				});
			}
			
			if (!bIsUniqueName)
			{
				bHasError = true;
				ErrorText = LOCTEXT("NonUniqueNameError", "Layout name must be unique");
			}
		}
		
		void CommitLayoutName(const FText& InNewName, ETextCommit::Type InCommitType)
		{
			SetLayoutName(InNewName);
			
			if (InCommitType == ETextCommit::OnEnter)
			{
				if (!EnteredLayoutName.IsEmpty() && !bHasError)
				{
					bConfirmed = true;
					CloseDialog();
				}
			}
		}
		
		bool IsConfirmButtonEnabled() const
		{
			return !EnteredLayoutName.IsEmpty() && !bHasError;
		}
		
		FReply OnConfirmButtonClicked()
		{
			bConfirmed = true;
			CloseDialog();
			return FReply::Handled();
		}
		
		FReply OnCancelButtonClicked()
		{
			CloseDialog();
			return FReply::Handled();
		}
		
	private:
		FString EnteredLayoutName;
		bool bConfirmed = false;
		
		FText ErrorText = FText::GetEmpty();
		bool bHasError = false;
	};
	
	FString ShowSaveLayoutAsDialog()
	{
		FString NewLayoutName = TEXT("");
		
		TSharedRef<SSaveLayoutAsDialog> Dialog = SNew(SSaveLayoutAsDialog);
		TSharedRef<SWindow> DialogWindow =
			SNew(SWindow)
			.Title(LOCTEXT("SaveLayoutAsWindowHeader", "Save Layout As"))
			.SizingRule(ESizingRule::Autosized);

		DialogWindow->SetContent(Dialog);
		FSlateApplication::Get().AddModalWindow(DialogWindow, FSlateApplication::Get().GetActiveTopLevelWindow());
		
		return Dialog->GetLayoutName();
	}
	
	bool ExportLayoutToString(const TSharedRef<FTabManager::FLayout>& InLayout, const FString& InNewLayoutName, FString& OutLayoutString)
	{
		FString SanitizedLayoutName = InNewLayoutName;
		for (TCHAR& InOutChar : SanitizedLayoutName)
		{
			if (!FChar::IsAlnum(InOutChar))
			{
				InOutChar = '_';
			}
		}
		
		const FString LayoutJsonName = "Standalone_MediaProfileEditor_" + SanitizedLayoutName;
		
		TSharedRef<FJsonObject> LayoutJson = InLayout->ToJson();
		LayoutJson->SetStringField(TEXT("Name"), LayoutJsonName);
		
		FString LayoutJsonString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&LayoutJsonString);
		if (!FJsonSerializer::Serialize(LayoutJson, Writer))
		{
			return false;
		}

		OutLayoutString = LayoutJsonString
			.Replace(TEXT("{"), TEXT("("))
			.Replace(TEXT("}"), TEXT(")"))
			.Replace(TEXT("\r"), TEXT(""))
			.Replace(TEXT("\n"), TEXT(""))
			.Replace(TEXT("\t"), TEXT(""));
		
		return true;
	}
	
	TSharedPtr<FTabManager::FLayout> ImportLayoutFromString(const FString& InLayoutString)
	{
		FString LayoutJson = InLayoutString
			.Replace(TEXT("("), TEXT("{"))
			.Replace(TEXT(")"), TEXT("}"))
			.Replace(TEXT("\\\n"), LINE_TERMINATOR)
			.Replace(TEXT("\\\r\n"), LINE_TERMINATOR);
		
		return FTabManager::FLayout::NewFromString(LayoutJson);
	}
}

TSharedRef<FMediaProfileEditor> FMediaProfileEditor::CreateMediaProfileEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UMediaProfile* InMediaProfile)
{
	TSharedRef<FMediaProfileEditor> NewEditor = MakeShared<FMediaProfileEditor>();
	NewEditor->Initialize(Mode, InitToolkitHost, InMediaProfile);
	return NewEditor;
}

void FMediaProfileEditor::Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UMediaProfile* InMediaProfile)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseOtherEditors(InMediaProfile, this);
	MediaProfileBeingEdited = InMediaProfile;

	FEditorDelegates::MapChange.AddSP(this, &FMediaProfileEditor::OnMapChange);
	GEngine->OnLevelActorDeleted().AddSP(this, &FMediaProfileEditor::OnLevelActorsRemoved);
	FEditorDelegates::OnAssetsDeleted.AddSP(this, &FMediaProfileEditor::OnAssetsDeleted);
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddSP(this, &FMediaProfileEditor::OnObjectPreEditChange);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FMediaProfileEditor::OnObjectPropertyChanged);
	
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = GetDefaultLayout();

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	InitAssetEditor(Mode, InitToolkitHost, AppName, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InMediaProfile);

	TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateSP(this, &FMediaProfileEditor::OnPersistLayout));
	
	BindCommands();
	ExtendToolbar();
	RegenerateMenusAndToolbars();
	ExtendWindowMenu();

	// When hosted in a custom application (e.g. Live Link Hub), the host provides its own Help menu.
	// Remove the auto-generated one to avoid command binding conflicts with the host's Help entries.
	{
		FString HostToolbarMenu;
		GConfig->GetString(TEXT("MediaProfile"), TEXT("ButtonMenu"), HostToolbarMenu, GEngineIni);
		if (!HostToolbarMenu.IsEmpty() && HostToolbarMenu != TEXT("LevelEditor.LevelEditorToolBar.User"))
		{
			UToolMenus::Get()->RemoveMenu(*(GetToolMenuName().ToString() + TEXT(".Help")));
		}
	}
	
	// TODO: Not sure this is the best place for this, but for now, when we open a media profile, automatically make it the active media profile
	// First, update the media profile settings config to match the media profile's source and output count.
	// Need to clear out current media profile first so that the change in proxy count does not propagate to whatever current media profile is configured
	IMediaProfileManager::Get().SetCurrentMediaProfile(nullptr);
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(InMediaProfile->NumMediaSources());
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(InMediaProfile->NumMediaOutputs());
	IMediaProfileManager::Get().SetCurrentMediaProfile(InMediaProfile);
	
	if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
	{
		if (!UserSettings->ActiveLayout.IsEmpty() && UserSettings->ActiveLayout != DefaultLayoutName)
		{
			LoadLayout(UserSettings->ActiveLayout);
		}
	}
}

FMediaProfileEditor::~FMediaProfileEditor()
{
	GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.RemoveAll(this);
	FEditorDelegates::OnAssetsDeleted.RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::PrePIEEnded.RemoveAll(this);
	FEditorDelegates::PostPIEStarted.RemoveAll(this);

	CloseAllMediaSources();
	CleanUpAndSaveUserSettings();

	// Remove Window menu entries if we added them
	if (bRegisteredWindowMenuEntries)
	{
		if (UToolMenus* ToolMenus = UToolMenus::Get())
		{
			ToolMenus->RemoveSection("MainFrame.MainMenu.Window", "MediaProfileEditor");
		}
	}
}

void FMediaProfileEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenuCategory", "Media Profile Editor"));
	
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(MediaOutputTabId, FOnSpawnTab::CreateSP(this, &FMediaProfileEditor::SpawnTab_MediaOutput))
		.SetDisplayName(LOCTEXT("MediaOutputTabDisplayName", "Media Output"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(MediaTreeTabId, FOnSpawnTab::CreateSP(this, &FMediaProfileEditor::SpawnTab_MediaTree))
		.SetDisplayName(LOCTEXT("MediaTreeTabDisplayName", "Media"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MediaPlayer"));

	InTabManager->RegisterTabSpawner(DetailsTabId, FOnSpawnTab::CreateSP(this, &FMediaProfileEditor::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("DetailsTabDisplayName", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(TimecodeTabId, FOnSpawnTab::CreateSP(this, &FMediaProfileEditor::SpawnTab_Timecode))
		.SetDisplayName(LOCTEXT("TimecodeTabDisplayName", "Timecode/Genlock"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), TEXT("ToolbarIcon.Timecode")));
}

void FMediaProfileEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(MediaOutputTabId);
	InTabManager->UnregisterTabSpawner(MediaTreeTabId);
	InTabManager->UnregisterTabSpawner(DetailsTabId);
	InTabManager->UnregisterTabSpawner(TimecodeTabId);
}

FName FMediaProfileEditor::GetToolkitFName() const
{
	return FName("MediaProfileEditor");
}

FText FMediaProfileEditor::GetBaseToolkitName() const
{
	return LOCTEXT("MediaProfileEditorLabel", "Media Profile Editor");
}

FText FMediaProfileEditor::GetToolkitName() const
{
	// Transient profiles have auto-generated names like "MediaProfile_0". Show a friendly generic
	// name instead. This is intentionally not unique — Live Link Hub only has a single media
	// profile at a time, so there is no ambiguity.
	if (IsEditingTransientProfile())
	{
		return LOCTEXT("TransientMediaProfileName", "Media Profile");
	}

	return FText::FromString(MediaProfileBeingEdited->GetName());
}

FText FMediaProfileEditor::GetToolkitToolTipText() const
{
	return FAssetEditorToolkit::GetToolTipTextForObject(MediaProfileBeingEdited);
}

FString FMediaProfileEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("MediaProfileEditor");
}

FLinearColor FMediaProfileEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

bool FMediaProfileEditor::IsEditingTransientProfile() const
{
	if (!MediaProfileBeingEdited)
	{
		return false;
	}

	// Check the object itself.
	if (MediaProfileBeingEdited->HasAnyFlags(RF_Transient))
	{
		return true;
	}

	// Check the package — mirrors the conditions in UObject::IsAsset().
	UPackage* Package = MediaProfileBeingEdited->GetPackage();
	return Package == GetTransientPackage()
		|| Package->HasAnyFlags(RF_Transient)
		|| Package->HasAnyPackageFlags(PKG_PlayInEditor);
}

bool FMediaProfileEditor::CanSaveAsset() const
{
	// Transient media profiles (e.g. in Live Link Hub) should not be saved as .uasset files.
	// Their state is persisted through the session config system instead.
	if (IsEditingTransientProfile())
	{
		return false;
	}

	return FAssetEditorToolkit::CanSaveAsset();
}

bool FMediaProfileEditor::IsSaveAssetVisible() const
{
	// Hide the save button entirely for transient profiles rather than showing it disabled.
	if (IsEditingTransientProfile())
	{
		return false;
	}

	return FAssetEditorToolkit::IsSaveAssetVisible();
}

bool FMediaProfileEditor::IsFindInContentBrowserButtonVisible() const
{
	// No asset to browse to for transient profiles.
	if (IsEditingTransientProfile())
	{
		return false;
	}

	return FAssetEditorToolkit::IsFindInContentBrowserButtonVisible();
}

UE::Editor::Toolbars::ECreateStatusBarOptions FMediaProfileEditor::GetStatusBarCreationOptions() const
{
	using namespace UE::Editor::Toolbars;

	// When editing a transient media profile, hide the Content Browser drawer since there are no
	// assets to browse. Also hide source control since the profile isn't persisted to disk.
	if (IsEditingTransientProfile())
	{
		return ECreateStatusBarOptions::HideContentBrowser | ECreateStatusBarOptions::HideSourceControl;
	}

	return FAssetEditorToolkit::GetStatusBarCreationOptions();
}

void FMediaProfileEditor::CloseAllMediaSources()
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}
	
	for (int32 Index = 0; Index < MediaProfileBeingEdited->NumMediaSources(); ++Index)
	{
		UMediaProfilePlaybackManager::FCloseSourceArgs Args;
		Args.Consumer = this;
		
		MediaProfileBeingEdited->GetPlaybackManager()->CloseSourceFromIndex(Index, Args);
	}
}

void FMediaProfileEditor::CloseAllMediaOutputs()
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}
	
	for (int32 MediaOutputIndex = 0; MediaOutputIndex < MediaProfileBeingEdited->NumMediaOutputs(); ++MediaOutputIndex)
	{
		MediaProfileBeingEdited->GetPlaybackManager()->CloseOutputFromIndex(MediaOutputIndex);
	}
}

bool FMediaProfileEditor::CanMediaOutputCapture(UMediaOutput* InMediaOutput) const
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return false;
	}
	
	UMediaProfileEditorCaptureSettings* CaptureSettings = GetMediaFrameworkCaptureSettings();
	if (!CaptureSettings)
	{
		return false;
	}

	// Return true if there is at least one valid capture configuration for the specified media output
	bool bHasValidCaptureSettings = false;
	
	using namespace UE::MediaFrameworkWorldSettings::Helpers;
	
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(CaptureSettings, InMediaOutput, 
		[this, &bHasValidCaptureSettings](const FMediaFrameworkCaptureCurrentViewportOutputInfo& OutputInfo)
		{
			//Check to see if there is a valid editor viewport that can be captured from
			TSharedPtr<FSceneViewport> Viewport = MediaProfileBeingEdited->GetPlaybackManager()->GetActiveViewport(OutputInfo.MediaOutput);
			bHasValidCaptureSettings = Viewport.IsValid();
		});
	
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(CaptureSettings, InMediaOutput, 
		[this, &bHasValidCaptureSettings](const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo)
		{
			const bool bHasValidCamera = OutputInfo.Cameras.ContainsByPredicate([](const TSoftObjectPtr<AActor>& ActorRef)
			{
				return ActorRef.IsValid();
			});
			
			if (bHasValidCamera)
			{
				bHasValidCaptureSettings = true;
			}
		});
	
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(CaptureSettings, InMediaOutput, 
		[this, &bHasValidCaptureSettings](const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& OutputInfo)
		{
			if (OutputInfo.RenderTarget)
			{
				bHasValidCaptureSettings = true;
			}
		});
	
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(CaptureSettings, InMediaOutput, 
		[this, &bHasValidCaptureSettings](const FMediaFrameworkCaptureMediaTextureOutputInfo& OutputInfo)
		{
			if (OutputInfo.MediaTexture)
			{
				bHasValidCaptureSettings = true;
			}
		});

	return bHasValidCaptureSettings;
}

UMediaProfileEditorCaptureSettings* FMediaProfileEditor::GetMediaFrameworkCaptureSettings()
{
	return GetMutableDefault<UMediaProfileEditorCaptureSettings>();
}

TSharedRef<FTabManager::FLayout> FMediaProfileEditor::GetDefaultLayout() const
{
	return FTabManager::NewLayout("Standalone_MediaProfileEditor_Layout")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->SetHideTabWell(true)
					->AddTab(MediaOutputTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.3f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.4f)
						->AddTab(MediaTreeTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->AddTab(DetailsTabId, ETabState::OpenedTab)
						->AddTab(TimecodeTabId, ETabState::OpenedTab)
					)
				)
			)
		);
}

TSharedRef<SDockTab> FMediaProfileEditor::SpawnTab_MediaOutput(const FSpawnTabArgs& InArgs)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("MediaOutputTabLabel", "Media Output"))
		[
			SAssignNew(ViewportPanel, SMediaProfileViewport, SharedThis(this))
		];
}

TSharedRef<SDockTab> FMediaProfileEditor::SpawnTab_MediaTree(const FSpawnTabArgs& InArgs)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("MediaTreeTabLabel", "Media"))
		[
			SNew(SMediaProfileSourcesTreeView, MediaProfileBeingEdited)
			.OnMediaItemDeleted(this, &FMediaProfileEditor::OnMediaItemDeleted)
			.OnSelectedMediaItemsChanged(this, &FMediaProfileEditor::OnSelectedMediaItemsChanged)
		];
}

TSharedRef<SDockTab> FMediaProfileEditor::SpawnTab_Details(const FSpawnTabArgs& InArgs)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("DetailsTabLabel", "Details"))
		[
			SAssignNew(DetailsPanel, SMediaProfileDetailsPanel, SharedThis(this), MediaProfileBeingEdited)
			.OnRefresh(this, &FMediaProfileEditor::RefreshSelectedMediaItems)
		];
}

TSharedRef<SDockTab> FMediaProfileEditor::SpawnTab_Timecode(const FSpawnTabArgs& InArgs)
{
	return SNew(SDockTab)
		.Label(LOCTEXT("TimecodeTabLabel", "TC/Genlock"))
		.ToolTipText(LOCTEXT("TimecodeTabTooltip", "Timecode and Genlock settings"))
		[
			SNew(SMediaFrameworkTimecodeGenlockPanel)
				.MediaProfile(MediaProfileBeingEdited)
		];
}

void FMediaProfileEditor::OpenTab(FName InTabId)
{
	TabManager->TryInvokeTab(InTabId);
}

void FMediaProfileEditor::BindCommands()
{
	ToolkitCommands->MapAction(
		FMediaProfileCommands::Get().ClearCurrentMediaProfile,
		FExecuteAction::CreateSP(this, &FMediaProfileEditor::ClearCurrentMediaProfile),
		FCanExecuteAction::CreateSP(this, &FMediaProfileEditor::CanClearCurrentMediaProfile));
	
	ToolkitCommands->MapAction(
		FMediaProfileCommands::Get().SaveLayout, 
		FExecuteAction::CreateSP(this, &FMediaProfileEditor::SaveLayout),
		FCanExecuteAction::CreateSP(this, &FMediaProfileEditor::CanSaveLayout));
	
	ToolkitCommands->MapAction(
		FMediaProfileCommands::Get().SaveLayoutAs, 
		FExecuteAction::CreateSP(this, &FMediaProfileEditor::SaveLayoutAs),
		FCanExecuteAction::CreateSP(this, &FMediaProfileEditor::CanSaveLayoutAs));
	
	ToolkitCommands->MapAction(
		FMediaProfileCommands::Get().RemoveAllLayouts,
		FExecuteAction::CreateSP(this, &FMediaProfileEditor::RemoveAllLayouts));
	
	ToolkitCommands->MapAction(
		FMediaProfileCommands::Get().ToggleFullscreen,
		FExecuteAction::CreateSP(this, &FMediaProfileEditor::ToggleFullscreen),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMediaProfileEditor::IsFullscreen));
	
	ToolkitCommands->MapAction(
		FMediaProfileCommands::Get().Immersive,
		FExecuteAction::CreateSP(this, &FMediaProfileEditor::SetViewportImmersive));
}

void FMediaProfileEditor::ExtendToolbar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShared<FExtender>();

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.AddToolBarButton(
				FUIAction(FExecuteAction::CreateSP(this, &FMediaProfileEditor::MakeCurrentMediaProfile)),
				NAME_None,
				TAttribute<FText>::CreateSP(this, &FMediaProfileEditor::GetIsCurrentMediaProfileLabel),
				FText::GetEmpty(),
				FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), "ClassIcon.MediaProfile"));
				
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(this, &FMediaProfileEditor::CreateCurrentProfileMenu),
				TAttribute<FText>::CreateSP(this, &FMediaProfileEditor::GetIsCurrentMediaProfileLabel),
				FText::GetEmpty(),
				FSlateIcon(),
				true);
		})
	);
	
	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.AddSeparator();
			ToolbarBuilder.AddWidget(
				SNew(SComboButton)
				.OnGetMenuContent(this, &FMediaProfileEditor::CreateLayoutMenu)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.f, 0.f, 6.f, 0.f))
					[
						SNew(SBox)
						.WidthOverride(16.f)
						.HeightOverride(16.f)
						[
							SNew(SImage).Image(FAppStyle::Get().GetBrush("Icons.Layout"))
						]
					]
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(this, &FMediaProfileEditor::GetActiveLayoutText, FText::GetEmpty(), true)
					]
				]
			);
		})
	);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.AddSeparator();
			ToolbarBuilder.AddWidget(
				SAssignNew(TimecodeToolbarEntry, SMediaFrameworkTimecodeGenlockHeader)
				.Visibility_Lambda([]()
				{
					if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
					{
						return (UserSettings->bShowTimecodeInEditorToolbar || UserSettings->bShowGenlockInEditorToolbar) ? EVisibility::Visible : EVisibility::Collapsed;
					}

					return EVisibility::Visible;
				})
				.ShowTimecode_Lambda([]()
				{
					if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
					{
						return UserSettings->bShowTimecodeInEditorToolbar;
					}

					return true;
				})
				.ShowGenlock_Lambda([]()
				{
					if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
					{
						return UserSettings->bShowGenlockInEditorToolbar;
					}

					return true;
				})
				.IsButton(true)
				.OnOpenTimecodeTab(this, &FMediaProfileEditor::OpenTab, TimecodeTabId)
			);
		})
	);

	AddToolbarExtender(ToolbarExtender);
}

void FMediaProfileEditor::ExtendWindowMenu()
{
	// When hosted in a custom application (e.g. LiveLinkHub), the editor's own Window menu
	// is not accessible. Register our tab spawners in the host's Window menu so users can
	// reopen closed panels from the menu bar.
	FString ToolbarMenu;
	GConfig->GetString(TEXT("MediaProfile"), TEXT("ButtonMenu"), ToolbarMenu, GEngineIni);

	if (ToolbarMenu.IsEmpty() || ToolbarMenu == TEXT("LevelEditor.LevelEditorToolBar.User"))
	{
		return;
	}

	UToolMenu* WindowMenu = UToolMenus::Get()->ExtendMenu("MainFrame.MainMenu.Window");
	WindowMenu->AddDynamicSection("MediaProfileEditor", FNewSectionConstructChoice(FNewToolMenuDelegateLegacy::CreateSP(this, &FMediaProfileEditor::PopulateWindowMenu)));
	bRegisteredWindowMenuEntries = true;
}

void FMediaProfileEditor::PopulateWindowMenu(FMenuBuilder& MenuBuilder, UToolMenu*)
{
	GetTabManager()->PopulateLocalTabSpawnerMenu(MenuBuilder);
}

void FMediaProfileEditor::RefreshSelectedMediaItems(const TArray<int32>& InMediaSources,const TArray<int32>& InMediaOutputs)
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}

	UMediaProfilePlaybackManager* PlaybackManager = MediaProfileBeingEdited->GetPlaybackManager();
	
	for (int32 Index = 0; Index < InMediaSources.Num(); ++Index)
	{
		const int32 SrcIndex = InMediaSources[Index];

		if (PlaybackManager->IsSourceOpenFromIndex(SrcIndex))
		{
			UMediaProfilePlaybackManager::FCloseSourceArgs Args;
			Args.Consumer = this;
			Args.bForceClose = true;
		
			PlaybackManager->CloseSourceFromIndex(SrcIndex, Args);
			PlaybackManager->OpenSourceFromIndex(SrcIndex, this);
		}
	}
	
	for (int32 Index = 0; Index < InMediaOutputs.Num(); ++Index)
	{
		const int32 OutputIndex = InMediaOutputs[Index];
		if (UMediaOutput* MediaOutput = MediaProfileBeingEdited->GetMediaOutput(OutputIndex))
		{
			RestartActiveMediaCaptures(MediaOutput);
		}
	}
}

void FMediaProfileEditor::CleanUpAndSaveUserSettings()
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}
	
	UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>();
	if (!UserSettings)
	{
		return;
	}
	
	// Remove any per media item settings that reference media items no longer in the media profile
	TSet<FName> CurrentMediaItemNames;
	for (int32 Index = 0; Index < MediaProfileBeingEdited->NumMediaSources(); ++Index)
	{
		if (UMediaSource* MediaSource = MediaProfileBeingEdited->GetMediaSource(Index))
		{
			CurrentMediaItemNames.Add(MediaSource->GetFName());
		}
	}

	for (int32 Index = 0; Index < MediaProfileBeingEdited->NumMediaOutputs(); ++Index)
	{
		if (UMediaOutput* MediaOutput = MediaProfileBeingEdited->GetMediaOutput(Index))
		{
			CurrentMediaItemNames.Add(MediaOutput->GetFName());
		}
	}
	
	TSet<FName> StoredMediaItemNames;
	UserSettings->PerMediaItemSettings.GetKeys(StoredMediaItemNames);

	for (const FName& MediaItemName : StoredMediaItemNames)
	{
		if (!CurrentMediaItemNames.Contains(MediaItemName))
		{
			UserSettings->PerMediaItemSettings.Remove(MediaItemName);
		}
	}

	UserSettings->SaveConfig();
}

void FMediaProfileEditor::RestartActiveMediaCaptures(UMediaOutput* InMediaOutput)
{
	UMediaProfilePlaybackManager* PlaybackManager = MediaProfileBeingEdited->GetPlaybackManager();
	if (!PlaybackManager)
	{
		return;
	}
	
	if (!PlaybackManager->IsOutputCapturing(InMediaOutput))
	{
		return;
	}
	
	UMediaProfileEditorCaptureSettings* CaptureSettings = GetMediaFrameworkCaptureSettings();
	if (!CaptureSettings)
	{
		return;
	}
	
	using namespace UE::MediaFrameworkWorldSettings::Helpers;
				
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(CaptureSettings, InMediaOutput, 
		[&PlaybackManager, bAutoRestart = CaptureSettings->bAutoRestartCaptureOnChange](const FMediaFrameworkCaptureCurrentViewportOutputInfo& OutputInfo)
		{
			PlaybackManager->RestartActiveViewportOutput(OutputInfo.MediaOutput, OutputInfo.CaptureOptions, bAutoRestart);
		});

	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(CaptureSettings, InMediaOutput, 
		[&PlaybackManager, bAutoRestart = CaptureSettings->bAutoRestartCaptureOnChange](const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo)
		{
			PlaybackManager->RestartManagedViewportOutput(OutputInfo.MediaOutput, OutputInfo.CaptureOptions, bAutoRestart);
		});

	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(CaptureSettings, InMediaOutput, 
		[&PlaybackManager, bAutoRestart = CaptureSettings->bAutoRestartCaptureOnChange](const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& OutputInfo)
		{
			PlaybackManager->RestartRenderTargetOutput(OutputInfo.MediaOutput, OutputInfo.RenderTarget, OutputInfo.CaptureOptions, bAutoRestart);
		});

	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(CaptureSettings, InMediaOutput, 
		[&PlaybackManager, bAutoRestart = CaptureSettings->bAutoRestartCaptureOnChange](const FMediaFrameworkCaptureMediaTextureOutputInfo& OutputInfo)
		{
			PlaybackManager->RestartMediaTextureOutput(OutputInfo.MediaOutput, OutputInfo.MediaTexture, OutputInfo.CaptureOptions, OutputInfo.Transform, bAutoRestart);
		});
}

FText FMediaProfileEditor::GetIsCurrentMediaProfileLabel() const
{
	return IMediaProfileManager::Get().GetCurrentMediaProfile() == MediaProfileBeingEdited
		? LOCTEXT("CurrentProfileLabel", "Current")
		: LOCTEXT("MakeCurrentProfileLabel", "Make Current");
}

TSharedRef<SWidget> FMediaProfileEditor::CreateCurrentProfileMenu()
{
	FMenuBuilder MenuBuilder(true, ToolkitCommands);

	MenuBuilder.BeginSection("CurrentMediaProfile", LOCTEXT("CurrentMediaProfileSectionLabel", "Current Media Profile"));
	{
		MenuBuilder.AddMenuEntry(FMediaProfileCommands::Get().ClearCurrentMediaProfile);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FMediaProfileEditor::MakeCurrentMediaProfile()
{
	IMediaProfileManager::Get().SetCurrentMediaProfile(MediaProfileBeingEdited);
}

void FMediaProfileEditor::ClearCurrentMediaProfile()
{
	IMediaProfileManager::Get().SetCurrentMediaProfile(nullptr);
}

bool FMediaProfileEditor::CanClearCurrentMediaProfile() const
{
	return IMediaProfileManager::Get().GetCurrentMediaProfile() == MediaProfileBeingEdited;
}

TSharedRef<SWidget> FMediaProfileEditor::CreateLayoutMenu()
{
	FMenuBuilder MenuBuilder(true, ToolkitCommands);
					
	MenuBuilder.BeginSection("CurrentLayout");
	{
		MenuBuilder.AddMenuEntry(
			TAttribute<FText>::CreateSP(this, &FMediaProfileEditor::GetActiveLayoutText, LOCTEXT("CurrentLayoutLabel", "Current Layout"), false),
			FText::GetEmpty(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Layout"),
			FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([]{ return false; })));
	}
	MenuBuilder.EndSection();
	
	MenuBuilder.BeginSection("SaveLayout", LOCTEXT("SaveLayoutSectionLabel", "Save Layout"));
	{
		MenuBuilder.AddMenuEntry(FMediaProfileCommands::Get().SaveLayout);
		MenuBuilder.AddMenuEntry(FMediaProfileCommands::Get().SaveLayoutAs);
	}
	MenuBuilder.EndSection();
	
	MenuBuilder.BeginSection("LoadLayout", LOCTEXT("LoadLayoutSectionLabel", "Load Layout"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DefaultLayout", "Default Layout"),
			FText::GetEmpty(),
			FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::Get().GetStyleSetName(), "ToolbarIcon.LoadLayout"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FMediaProfileEditor::LoadLayout, DefaultLayoutName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FMediaProfileEditor::IsLayoutLoaded, DefaultLayoutName)),
			NAME_None,
			EUserInterfaceActionType::RadioButton);
		
		if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
		{
			for (const FMediaProfileEditorLayoutSettings& Layout : UserSettings->SavedLayouts)
			{
				MenuBuilder.AddMenuEntry(
					FText::FromString(Layout.Name),
					FText::GetEmpty(),
					FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::Get().GetStyleSetName(), "ToolbarIcon.LoadLayout"),
					FUIAction(
						FExecuteAction::CreateSP(this, &FMediaProfileEditor::LoadLayout, Layout.Name),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(this, &FMediaProfileEditor::IsLayoutLoaded, Layout.Name)),
					NAME_None,
					EUserInterfaceActionType::RadioButton);
			}
		}
	}
	MenuBuilder.EndSection();
	
	MenuBuilder.BeginSection("RemoveLayout", LOCTEXT("RemoveLayoutSectionLabel", "Remove Layout"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("RemoveLayoutSubMenuLabel", "Remove Layout"),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InSubMenuBuilder)
			{
				if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
				{
					for (const FMediaProfileEditorLayoutSettings& Layout : UserSettings->SavedLayouts)
					{
						InSubMenuBuilder.AddMenuEntry(
							FText::FromString(Layout.Name),
							FText::GetEmpty(),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateSP(this, &FMediaProfileEditor::RemoveLayout, Layout.Name)));
					}
				}
				
				InSubMenuBuilder.AddMenuSeparator();
				InSubMenuBuilder.AddMenuEntry(FMediaProfileCommands::Get().RemoveAllLayouts);
			}),
			false,
			FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::Get().GetStyleSetName(), "ToolbarIcon.RemoveLayout"));
	}
	MenuBuilder.EndSection();
	
	MenuBuilder.BeginSection("Immersive");
	{
		MenuBuilder.AddMenuSeparator();
		MenuBuilder.AddMenuEntry(FMediaProfileCommands::Get().ToggleFullscreen);
	}
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

FText FMediaProfileEditor::GetActiveLayoutText(FText Prefix, bool bDirtyFlag) const
{
	if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
	{
		FString ActiveLayout = !UserSettings->ActiveLayout.IsEmpty() ? UserSettings->ActiveLayout : DefaultLayoutName;
		
		
		FText ActiveLayoutLabel = FText::FromString(ActiveLayout);
		if (bDirtyFlag && ActiveLayout != DefaultLayoutName && bActiveLayoutDirty)
		{
			ActiveLayoutLabel = bActiveLayoutDirty ? FText::Format(LOCTEXT("DirtyLayoutLabelFormat", "{0}*"), FText::FromString(ActiveLayout)) : FText::FromString(ActiveLayout);
		}
		
		return !Prefix.IsEmpty()
			? FText::Format(LOCTEXT("CurrentLayoutMenuEntryFormat", "{0}: {1}"), Prefix, ActiveLayoutLabel)
			: ActiveLayoutLabel;
	}
	
	return FText::GetEmpty();
}

void FMediaProfileEditor::SaveLayout()
{
	if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
	{
		const FString& ActiveLayout = UserSettings->ActiveLayout;
		FMediaProfileEditorLayoutSettings* ExistingLayout = 
			UserSettings->SavedLayouts.FindByPredicate([&ActiveLayout](const FMediaProfileEditorLayoutSettings& Layout) { return Layout.Name == ActiveLayout; });
		
		if (!ExistingLayout)
		{
			return;
		}

		if (!UE::MediaProfileEditor::Private::ExportLayoutToString(GetTabManager()->PersistLayout(), ActiveLayout, ExistingLayout->Layout))
		{
			return;
		}
		
		if (ViewportPanel.IsValid())
		{
			ViewportPanel->SaveLayout(*ExistingLayout);
		}
		
		UserSettings->SaveConfig();
		
		bActiveLayoutDirty = false;
	}
}

bool FMediaProfileEditor::CanSaveLayout() const
{
	if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
	{
		return
			!UserSettings->ActiveLayout.IsEmpty() && 
			UserSettings->ActiveLayout != DefaultLayoutName && 
			UserSettings->SavedLayouts.ContainsByPredicate([Name = UserSettings->ActiveLayout](const FMediaProfileEditorLayoutSettings& LayoutSettings)
			{
				return LayoutSettings.Name == Name;
			});
	}
	
	return false;
}

void FMediaProfileEditor::SaveLayoutAs()
{
	const FString NewLayoutName = UE::MediaProfileEditor::Private::ShowSaveLayoutAsDialog();
	if (NewLayoutName.IsEmpty())
	{
		return;
	}
	
	if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
	{
		FMediaProfileEditorLayoutSettings NewLayout;
		NewLayout.Name = NewLayoutName;
		
		if (!UE::MediaProfileEditor::Private::ExportLayoutToString(GetTabManager()->PersistLayout(), NewLayoutName, NewLayout.Layout))
		{
			return;
		}
		
		if (ViewportPanel.IsValid())
		{
			ViewportPanel->SaveLayout(NewLayout);
		}
		
		UserSettings->SavedLayouts.Add(NewLayout);
		UserSettings->SaveConfig();
		
		LoadLayout(NewLayoutName);
	}
}

bool FMediaProfileEditor::CanSaveLayoutAs() const
{
	if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
	{
		return
			UserSettings->ActiveLayout.IsEmpty() ||
			UserSettings->ActiveLayout == DefaultLayoutName;
	}
	
	return false;
}

void FMediaProfileEditor::LoadLayout(FString LayoutName)
{
	if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
	{
		if (LayoutName.IsEmpty() || LayoutName == DefaultLayoutName)
		{
			// Don't load any saved modifications to the default layout to ensure the layout gets set to true default
			const bool bLoadUserLayout = false;
			RestoreFromLayout(GetDefaultLayout(), bLoadUserLayout);
			RegenerateMenusAndToolbars();
		
			if (ViewportPanel.IsValid())
        	{
				ViewportPanel->ClearLayout();
			}
		}
		else
		{
			const FMediaProfileEditorLayoutSettings* ExistingLayout = 
				UserSettings->SavedLayouts.FindByPredicate([&LayoutName](const FMediaProfileEditorLayoutSettings& Layout) { return Layout.Name == LayoutName; });
			if (!ExistingLayout)
			{
				return;
			}
	
			if (TSharedPtr<FTabManager::FLayout> NewLayout = UE::MediaProfileEditor::Private::ImportLayoutFromString(ExistingLayout->Layout))
			{
				const bool bLoadUserLayout = false;
				RestoreFromLayout(NewLayout.ToSharedRef(), bLoadUserLayout);
				RegenerateMenusAndToolbars();
			
				if (ViewportPanel.IsValid())
				{
					ViewportPanel->LoadLayout(*ExistingLayout);
				}
			}
		}
		
		UserSettings->ActiveLayout = LayoutName;
		UserSettings->SaveConfig();
		
		bActiveLayoutDirty = false;
	}
}

bool FMediaProfileEditor::IsLayoutLoaded(FString LayoutName) const
{
	if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
	{
		const bool bIsDefaultLayout = LayoutName.IsEmpty() || LayoutName == DefaultLayoutName;
		return bIsDefaultLayout
			? UserSettings->ActiveLayout.IsEmpty() || UserSettings->ActiveLayout == DefaultLayoutName
			: UserSettings->ActiveLayout == LayoutName;
	}
	
	return false;
}

void FMediaProfileEditor::RemoveLayout(FString LayoutName)
{
	if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
	{
		const int32 ExistingLayoutIndex = 
				UserSettings->SavedLayouts.IndexOfByPredicate([&LayoutName](const FMediaProfileEditorLayoutSettings& Layout) { return Layout.Name == LayoutName; });
		if (ExistingLayoutIndex == INDEX_NONE)
		{
			return;
		}
		
		if (UserSettings->ActiveLayout == UserSettings->SavedLayouts[ExistingLayoutIndex].Name)
		{
			LoadLayout(DefaultLayoutName);
		}
		
		UserSettings->SavedLayouts.RemoveAt(ExistingLayoutIndex);
		UserSettings->SaveConfig();
	}
}

void FMediaProfileEditor::RemoveAllLayouts()
{
	if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
	{
		if (!UserSettings->ActiveLayout.IsEmpty() && UserSettings->ActiveLayout != DefaultLayoutName)
		{
			LoadLayout(DefaultLayoutName);
		}
		
		UserSettings->SavedLayouts.Empty();
		UserSettings->SaveConfig();
	}
}

void FMediaProfileEditor::ToggleFullscreen()
{
	TSharedPtr<SDockTab> Tab = TabManager->GetOwnerTab();
	const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(Tab.ToSharedRef());

	if (Window->GetWindowMode() == EWindowMode::Windowed)
	{
		Window->SetWindowMode(EWindowMode::WindowedFullscreen);
	}
	else
	{
		Window->SetWindowMode(EWindowMode::Windowed);
	}
}

bool FMediaProfileEditor::IsFullscreen() const
{
	TSharedPtr<SDockTab> Tab = TabManager->GetOwnerTab();
	const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(Tab.ToSharedRef());

	return Window->GetWindowMode() == EWindowMode::WindowedFullscreen;
}

void FMediaProfileEditor::SetViewportImmersive()
{
	if (ViewportPanel.IsValid())
	{
		ViewportPanel->SetImmersivePanel();
	}
}

void FMediaProfileEditor::OnPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave)
{
	bActiveLayoutDirty = true;
	FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
}

void FMediaProfileEditor::OnMediaItemDeleted(UClass* InMediaType, int32 InMediaItemIndex)
{
	if (InMediaType->IsChildOf<UMediaSource>())
	{
		UMediaProfilePlaybackManager::FCloseSourceArgs Args;
		Args.Consumer = this;
		Args.bDestroyMediaPlayer = true;
		Args.bForceClose = true;
		
		MediaProfileBeingEdited->GetPlaybackManager()->CloseSourceFromIndex(InMediaItemIndex, Args);
	}
	else if (InMediaType->IsChildOf<UMediaOutput>())
	{
		MediaProfileBeingEdited->GetPlaybackManager()->CloseOutputFromIndex(InMediaItemIndex);
	}

	if (ViewportPanel.IsValid())
	{
		ViewportPanel->ForceClearMediaItem(InMediaType, InMediaItemIndex);
	}
}

void FMediaProfileEditor::OnSelectedMediaItemsChanged(const TArray<int32>& SelectedMediaSources, const TArray<int32>& SelectedMediaOutputs)
{
	if (!DetailsPanel.IsValid())
	{
		return;
	}

	ViewportPanel->SetSelectedMediaItems(SelectedMediaSources, SelectedMediaOutputs);
	DetailsPanel->SetSelectedMediaItems(SelectedMediaSources, SelectedMediaOutputs);
}

void FMediaProfileEditor::OnMapChange(uint32 InMapFlags)
{
	// New map might have different capture settings, so stop all captures and notify of settings changes
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}

	CloseAllMediaOutputs();
}

void FMediaProfileEditor::OnLevelActorsRemoved(AActor* InActor)
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}
	
	UMediaProfileEditorCaptureSettings* CaptureSettings = GetMediaFrameworkCaptureSettings();
	if (!CaptureSettings)
	{
		return;
	}
	
	// If the removed actor is referenced by one of the live captures (e.g. a camera in a viewport capture) we need to stop the capture
	for (const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& ViewportOutputInfo : CaptureSettings->ViewportCaptures)
	{
		if (ViewportOutputInfo.Cameras.Contains(InActor))
		{
			MediaProfileBeingEdited->GetPlaybackManager()->CloseOutput(ViewportOutputInfo.MediaOutput);
		}
	}
}

void FMediaProfileEditor::OnAssetsDeleted(const TArray<UClass*>& DeletedAssetClasses)
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}
	
	UMediaProfileEditorCaptureSettings* CaptureSettings = GetMediaFrameworkCaptureSettings();
	if (!CaptureSettings)
	{
		return;
	}

	bool bContainsRenderTargets = DeletedAssetClasses.ContainsByPredicate([](const UClass* Class)
	{
		return Class->IsChildOf<UTextureRenderTarget2D>();
	});
	
	if (bContainsRenderTargets)
	{
		// If the deleted asset is a render target, we need to stop all render target captures in case the deleted render target is used in one of them
		for (const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& RenderTargetOutputInfo : CaptureSettings->RenderTargetCaptures)
		{
			MediaProfileBeingEdited->GetPlaybackManager()->CloseOutput(RenderTargetOutputInfo.MediaOutput);
		}
	}
}

void FMediaProfileEditor::OnObjectPreEditChange(UObject* Object, const FEditPropertyChain& PropertyChain)
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}
	
	// If the capture settings are about to be changed, stop all captures
	UMediaProfileEditorCaptureSettings* CaptureSettings = GetMediaFrameworkCaptureSettings();
	if (Object == CaptureSettings)
	{
		CloseAllMediaOutputs();
	}
}

void FMediaProfileEditor::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (!IsValid(MediaProfileBeingEdited))
	{
		return;
	}
	
	if (!InObject)
	{
		return;
	}

	if (MediaProfileBeingEdited != InObject)
	{
		UMediaProfilePlaybackManager* PlaybackManager = MediaProfileBeingEdited->GetPlaybackManager();
		
		if (UMediaSource* MediaSource = Cast<UMediaSource>(InObject))
		{
			const int32 SrcIndex = MediaProfileBeingEdited->FindMediaSourceIndex(MediaSource);
			if (SrcIndex != INDEX_NONE)
			{
				if (PlaybackManager->IsSourceOpenFromIndex(SrcIndex))
				{
					bool bPropertyChangeRequiresRestart = true;

					if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UCaptureCardMediaSource, SourceColorSettings))
					{
						bPropertyChangeRequiresRestart = false;
					}

					if (bPropertyChangeRequiresRestart)
					{
						UMediaProfilePlaybackManager::FCloseSourceArgs Args;
						Args.Consumer = this;
						Args.bForceClose = true;

						PlaybackManager->CloseSourceFromIndex(SrcIndex, Args);
						PlaybackManager->OpenSourceFromIndex(SrcIndex, this);
					}
				}
			}
		}

		if (UMediaOutput* MediaOutput = Cast<UMediaOutput>(InObject))
		{
			const int32 OutputIndex = MediaProfileBeingEdited->FindMediaOutputIndex(MediaOutput);
			if (OutputIndex != INDEX_NONE)
			{
				RestartActiveMediaCaptures(MediaOutput);
			}
		}
	}

	if (MediaProfileBeingEdited == InObject)
	{
		if (GetDefault<UMediaProfileSettings>()->bAutosaveAfterChange)
		{
			if (!IsEditingTransientProfile())
			{
				SaveAsset_Execute();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
