// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorViewportClient.h"
#include "Toolkits/AssetEditorToolkit.h"

enum class EMediaCaptureState : uint8;
class SMediaFrameworkTimecodeGenlockHeader;
class SMediaProfileViewport;
class FMediaFrameworkCaptureLevelEditorViewportClient;
class UMediaFrameworkWorldSettingsAssetUserData;
class UMediaOutput;
class UMediaCapture;
class UMediaProfile;
class UMediaProfileEditorCaptureSettings;
class SMediaProfileDetailsPanel;

/**
 * Media profile editor 2.0, which replaces the old, simple editor with a more robust one
 */
class FMediaProfileEditor : public FAssetEditorToolkit
{
private:
	static const FName AppName;
	static const FName MediaOutputTabId;
	static const FName MediaTreeTabId;
	static const FName DetailsTabId;
	static const FName TimecodeTabId;
	
	static const FString DefaultLayoutName;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCaptureMethodChanged, UMediaOutput*)

public:
	static TSharedRef<FMediaProfileEditor> CreateMediaProfileEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMediaProfile* InMediaProfile);

	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UMediaProfile* InMediaProfile);

	virtual ~FMediaProfileEditor() override;
	
	// IToolkit interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	// End of IToolkit interface

	// FAssetEditorToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual bool CanSaveAsset() const override;
	virtual bool IsSaveAssetVisible() const override;
	virtual bool IsFindInContentBrowserButtonVisible() const override;
	virtual UE::Editor::Toolbars::ECreateStatusBarOptions GetStatusBarCreationOptions() const override;
	// End of FAssetEditorToolkit

	/** Gets the media profile being edited */
	UMediaProfile* GetMediaProfile() const { return MediaProfileBeingEdited; }

	/** Returns true if the profile being edited is a transient (in-memory) profile, e.g. in Live Link Hub. */
	bool IsEditingTransientProfile() const;

	/** Closes all open media sources in the media profile */
	void CloseAllMediaSources();

	/** Closes all open media outputs in the media profile */
	void CloseAllMediaOutputs();
	
	/** Determines if the specified media output is properly configured for capturing */
	bool CanMediaOutputCapture(UMediaOutput* InMediaOutput) const;

	/** Restart any active media captures for the specified media output */
	void RestartActiveMediaCaptures(UMediaOutput* InMediaOutput);
	
	/** Gets the media capture settings config object */
	static UMediaProfileEditorCaptureSettings* GetMediaFrameworkCaptureSettings();

	/** Gets the delegate that is raised when the capture method is changed through the details panel for a media output */
	FOnCaptureMethodChanged& GetOnCaptureMethodChanged() { return OnCaptureMethodChanged; }
	
private:
	/** Creates the default layout to use for the media profile editor */
	TSharedRef<FTabManager::FLayout> GetDefaultLayout() const;
	
	TSharedRef<SDockTab> SpawnTab_MediaOutput(const FSpawnTabArgs& InArgs);
	TSharedRef<SDockTab> SpawnTab_MediaTree(const FSpawnTabArgs& InArgs);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& InArgs);
	TSharedRef<SDockTab> SpawnTab_Timecode(const FSpawnTabArgs& InArgs);

	/** Raised when something has requested that a tab be manually opened/focused */
	void OpenTab(FName InTabId);

	/** Binds commands to the asset editor's command list */
	void BindCommands();
	
	void ExtendToolbar();

	/** Registers the editor's tab spawners in the host application's Window menu (e.g. LiveLinkHub) */
	void ExtendWindowMenu();

	/** Populates the host Window menu with this editor's tab spawners */
	void PopulateWindowMenu(class FMenuBuilder& MenuBuilder, class UToolMenu*);

	/** Refreshes the selected media items, reopening media sources and outputs */
	void RefreshSelectedMediaItems(const TArray<int32>& InMediaSources,const TArray<int32>& InMediaOutputs);

	/** Cleans up and saves the media profile editor user settings config */
	void CleanUpAndSaveUserSettings();
	
	/** Gets the label used to indicate if the media profile being edited is set as the current editor media profile */
	FText GetIsCurrentMediaProfileLabel() const;

	/** Creates the dropdown menu for the "Current" profile toolbar button */
	TSharedRef<SWidget> CreateCurrentProfileMenu();

	/** Makes the media profile being edited the currently active media profile */
	void MakeCurrentMediaProfile();
	
	/** Clears the media profile being edited from being the currently active media profile */
	void ClearCurrentMediaProfile();

	/** Gets whether the media profile being edited can be cleared from being the currently active media profile */
	bool CanClearCurrentMediaProfile() const;

	TSharedRef<SWidget> CreateLayoutMenu();

	/** Gets the text to display in the layouts menu for the currently active layout */
	FText GetActiveLayoutText(FText Prefix, bool bDirtyFlag) const;

	/** Raised when the 'Save Layout' menu entry is clicked in the Layouts dropdown */
	void SaveLayout();

	/** Gets whether the 'Save Layout' menu entry can be used on the current layout */
	bool CanSaveLayout() const;

	/** Raised when the 'Save Layout As' menu entry is clicked in the Layouts dropdown */
	void SaveLayoutAs();

	/** Gets whether the 'Save Layout As' menu entry can be used on the current layout */
	bool CanSaveLayoutAs() const;

	/** Raised when a specific layout is selected to load in the Layouts dropdown */
	void LoadLayout(FString LayoutName);

	/** Gets whether the layout of the specified name is the currently active layout */
	bool IsLayoutLoaded(FString LayoutName) const;

	/** Removes the layout of the specified name from the list of saved layouts */
	void RemoveLayout(FString LayoutName);

	/** Removes all saved layouts */
	void RemoveAllLayouts();

	/** Toggles fullscreen mode on the window that contains the media profile editor */
	void ToggleFullscreen();

	/** Gets whether the window containing the media profile editor is in fullscreen mode or not */
	bool IsFullscreen() const;

	/** Sets the media profile editor viewport to immersive mode */
	void SetViewportImmersive();

	/** Raised when the tab manager for this editor tries to save the current tab layout to config */
	void OnPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave);

	void OnMediaItemDeleted(UClass* InMediaType, int32 InMediaItemIndex);
	void OnSelectedMediaItemsChanged(const TArray<int32>& SelectedMediaSources, const TArray<int32>& SelectedMediaOutputs);
	
	void OnMapChange(uint32 InMapFlags);
	void OnLevelActorsRemoved(AActor* InActor);
	void OnAssetsDeleted(const TArray<UClass*>& DeletedAssetClasses);
	void OnObjectPreEditChange(UObject* Object, const FEditPropertyChain& PropertyChain);
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
	
private:
	/** The media profile being edited by this editor */
	TObjectPtr<UMediaProfile> MediaProfileBeingEdited;

	TSharedPtr<SMediaProfileDetailsPanel> DetailsPanel;
	TSharedPtr<SMediaProfileViewport> ViewportPanel;
	
	/** Timecode/Genlock header widget that is displayed in the editor's primary toolbar */
	TSharedPtr<SMediaFrameworkTimecodeGenlockHeader> TimecodeToolbarEntry;

	/** List of media outputs whose capture needs to be restarted on the next engine tick */
	TSet<TWeakObjectPtr<UMediaOutput>> OutputsToRestartCapture;

	/** Indicates if the currently active layout has been dirtied and can be saved */
	bool bActiveLayoutDirty = false;
	
	/** Raised when the capture method is changed on a media output through the details panel */
	FOnCaptureMethodChanged OnCaptureMethodChanged;
	
	/** Handle for callback to restart any modified captures on the next engine tick */
	FTimerHandle RestartCapturesTimerHandle;

	/** Whether this editor registered entries in the host application's Window menu */
	bool bRegisteredWindowMenuEntries = false;
};
