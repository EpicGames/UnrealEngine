// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "MediaFrameworkWorldSettingsAssetUserData.h"

#include "MediaProfileEditorUserSettings.generated.h"

/** Rotation of the displayed media image in the media profile editor */
UENUM()
enum class EMediaImageRotation
{
	None UMETA(DisplayName = "0°"),
	CW90 UMETA(DisplayName = "90°"),
	CCW90 UMETA(DisplayName = "-90°"),
	CW180 UMETA(DisplayName = "180°")
};

/** Settings for a media source/output in the media profile editor */
USTRUCT()
struct FMediaProfileEditorPerMediaItemSettings
{
	GENERATED_BODY()

public:
	/** The saved color channel mask applied to the media source/output */
	UPROPERTY(Config)
	int32 ColorChannelMask = INDEX_NONE;

	/** The saved invert alpha flag applied to the media source/output */
	UPROPERTY(Config)
	bool bInvertAlphaChannelMask = false;

	/** The saved draw checkerboard flag applied to the media source/output */
	UPROPERTY(Config)
	bool bDrawAlphaBlendedCheckerboard = false;
	
	/** The rotation of the media image in the viewport */
	UPROPERTY(Config)
	EMediaImageRotation Rotation = EMediaImageRotation::None;
	
	/** Whether the media image is flipped horizontally */
	UPROPERTY(Config)
	bool bFlipHorizontal = false;
	
	/** Whether the media image is flipped vertically */
	UPROPERTY(Config)
	bool bFlipVertical = false;
};

USTRUCT()
struct FMediaProfileEditorLayoutPanelSettings
{
	GENERATED_BODY()
	
	/** The name of the media item being displayed in the panel */
	UPROPERTY(Config)
	FString MediaItemName;
	
	/** Whether the media item being displayed is a media source or media output */
	UPROPERTY(Config)
	bool bMediaSource = false;
};

USTRUCT()
struct FMediaProfileEditorLayoutSettings
{
	GENERATED_BODY()
	
	/** The name of the saved layout */
	UPROPERTY(Config)
	FString Name;
	
	/** The stringified FTabManager::FLayout of the saved layout */
	UPROPERTY(Config)
	FString Layout;
	
	/** A flag that indicates which panel arrangement the viewport has for the layout */
	UPROPERTY(Config)
	uint8 ViewportLayout = 0;

	/** A flag that indicates which panel orientation the viewport has for the layout */
	UPROPERTY(Config)
	uint8 ViewportOrientation = 0;
	
	/** A list of panel contents the viewport has for the layout */
	UPROPERTY(Config)
	TArray<FMediaProfileEditorLayoutPanelSettings> Panels; 
};

/** Config class to store editor user settings for the media profile editor */
UCLASS(MinimalAPI, Config=EditorPerProjectUserSettings)
class UMediaProfileEditorUserSettings : public UObject
{
	GENERATED_BODY()
	
public:
	/** Whether to show the timecode entry in the viewport toolbar */
	UPROPERTY(Config)
	bool bShowTimecodeInViewportToolbar = true;

	/** Whether to show the timecode entry in the main editor toolbar */
	UPROPERTY(Config)
	bool bShowTimecodeInEditorToolbar = true;
	
	/** Whether to show the genlock entry in the viewport toolbar */
	UPROPERTY(Config)
	bool bShowGenlockInViewportToolbar = true;

	/** Whether to show the genlock entry in the main editor toolbar */
	UPROPERTY(Config)
	bool bShowGenlockInEditorToolbar = true;
	
	/** Whether to show the Channels menu in the viewports */
	UPROPERTY(Config)
	bool bShowChannelsInViewportToolbar = true;

	/** Config settings specific to individual media sources/outputs, indexed by the media item's name */
	UPROPERTY(Config)
	TMap<FName, FMediaProfileEditorPerMediaItemSettings> PerMediaItemSettings;
	
	/** A list of user-created layouts for the media profile editor */
	UPROPERTY(Config)
	TArray<FMediaProfileEditorLayoutSettings> SavedLayouts;
	
	/** The currently active layout for the media profile editor */
	UPROPERTY(Config)
	FString ActiveLayout;
};

/**
 * Variant of the media capture settings config for media outputs in the media profile
 */
UCLASS(MinimalAPI, config = EditorSettings)
class UMediaProfileEditorCaptureSettings : public UMediaFrameworkWorldSettingsAssetUserData
{
	GENERATED_BODY()
	
public:
	/** Should the capture be restarted if the media output is modified. */
	UPROPERTY(config)
	bool bAutoRestartCaptureOnChange = true;
};