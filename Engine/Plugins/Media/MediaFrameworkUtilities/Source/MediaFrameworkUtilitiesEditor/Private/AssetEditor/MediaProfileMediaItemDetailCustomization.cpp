// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaProfileMediaItemDetailCustomization.h"

#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "IStructureDataProvider.h"
#include "MediaFrameworkWorldSettingsAssetUserData.h"
#include "MediaPlayer.h"
#include "MediaProfileEditorUserSettings.h"
#include "MediaTexture.h"
#include "SMediaProfileSourceTexturePicker.h"
#include "StreamMediaSource.h"
#include "Internationalization/Culture.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "FMediaProfileMediaItemDetailCustomization"

void FMediaProfileMediaSourceDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FMediaProfileMediaItemDetailCustomization<UMediaSource>::CustomizeDetails(DetailBuilder);

	TArray<TWeakObjectPtr<UMediaSource>> Sources = CachedDetailBuilder.Pin()->GetObjectsOfTypeBeingCustomized<UMediaSource>();

	if (Sources.Num() == 1)
	{
		TStrongObjectPtr<UMediaSource> Source = Sources[0].Pin();
		if (Source.IsValid() && ShowMediaPlayerOptions(Source.Get()))
		{
			AddMediaPlayerProperties(Source.Get(), DetailBuilder);
		}
	}
}

FText FMediaProfileMediaSourceDetailCustomization::GetMediaItemTypeText() const
{
	return LOCTEXT("MediaSourceTypeText", "Media Source");
}

void FMediaProfileMediaSourceDetailCustomization::RegisterMediaTypeSection(FPropertyEditorModule& PropertyModule, const FName& MediaTypeCategory)
{
	static bool bRegistered = false;
	if (!bRegistered)
	{
		const TSharedPtr<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			UMediaSource::StaticClass()->GetFName(),
			TEXT("MediaSourceSection"),
			GetMediaItemTypeText());

		Section->AddCategory(MediaTypeCategory);
		bRegistered = true;
	}
}

FText FMediaProfileMediaSourceDetailCustomization::GetMediaObjectLabel(UMediaSource* InMediaObject) const
{
	if (!MediaProfile.IsValid())
	{
		return FText::GetEmpty();
	}

	int32 Index = MediaProfile->FindMediaSourceIndex(InMediaObject);
	if (InMediaObject->IsA<UDummyMediaSource>())
	{
		Index = Cast<UDummyMediaSource>(InMediaObject)->MediaProfileIndex;
	}
		
	return FText::FromString(MediaProfile->GetLabelForMediaSource(Index));
}

void FMediaProfileMediaSourceDetailCustomization::SetMediaObjectLabel(UMediaSource* InMediaObject, const FText& InLabel)
{
	if (!MediaProfile.IsValid())
	{
		return;
	}
	
	int32 Index = MediaProfile->FindMediaSourceIndex(InMediaObject);
	if (InMediaObject->IsA<UDummyMediaSource>())
	{
		Index = Cast<UDummyMediaSource>(InMediaObject)->MediaProfileIndex;
	}
		
	MediaProfile->SetLabelForMediaSource(Index, InLabel.ToString());
}

void FMediaProfileMediaSourceDetailCustomization::SetMediaObject(UMediaSource* InOriginalMediaObject, UMediaSource* InNewMediaObject)
{
	if (!MediaProfile.IsValid())
    {
    	return;
    }
    	
	int32 Index = MediaProfile->FindMediaSourceIndex(InOriginalMediaObject);
	if (InOriginalMediaObject->IsA<UDummyMediaSource>())
	{
		Index = Cast<UDummyMediaSource>(InOriginalMediaObject)->MediaProfileIndex;
	}
		
	MediaProfile->SetMediaSource(Index, InNewMediaObject);
}

bool FMediaProfileMediaSourceDetailCustomization::ShowMediaPlayerOptions(UMediaSource* InSource) const
{
	// TODO: Might need something more sophisticated in the future (something queryable on the source type, like a class attribute)
	// if more media source types need to display the media player properties
	if (InSource->IsA<UStreamMediaSource>())
	{
		return true;
	}

	return false;
}

class SIndexComboBox : public SComboBox<TSharedPtr<int32>>
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetIndexDisplayText, const int32);
	DECLARE_DELEGATE_TwoParams(FOnIndexChanged, int32, ESelectInfo::Type);
	
	SLATE_BEGIN_ARGS(SIndexComboBox) { }
		SLATE_ATTRIBUTE(int32, NumIndices)
		SLATE_ATTRIBUTE(int32, StartingIndex)
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_ATTRIBUTE(int32, SelectedIndex)
		SLATE_EVENT(FGetIndexDisplayText, GetIndexDisplayText)
		SLATE_EVENT(FOnIndexChanged, OnIndexChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		NumIndicesAttr = InArgs._NumIndices;
		StartingIndexAttr = InArgs._StartingIndex;
		SelectedIndexAttr = InArgs._SelectedIndex;
		GetIndexDisplayText = InArgs._GetIndexDisplayText;
		OnIndexChanged = InArgs._OnIndexChanged;
		
		SComboBox<TSharedPtr<int32>>::FArguments BaseArgs;
		BaseArgs.OptionsSource(&Indices);
		BaseArgs.OnGenerateWidget_Lambda([WeakThis = AsWeak()](TSharedPtr<int32> Item)->TSharedRef<SWidget>
		{
			TSharedPtr<SIndexComboBox> This = StaticCastSharedPtr<SIndexComboBox>(WeakThis.Pin());
			if (!This.IsValid())
			{
				return SNullWidget::NullWidget;
			}
			
			const int32 Index = *Item;
			return SNew(STextBlock)
				.Text(TAttribute<FText>::CreateLambda([WeakThis, Index]
				{
					TSharedPtr<SIndexComboBox> This = StaticCastSharedPtr<SIndexComboBox>(WeakThis.Pin());
					if (!This.IsValid())
					{
						return FText::GetEmpty();
					}
					
					if (This->GetIndexDisplayText.IsBound())
					{
						return This->GetIndexDisplayText.Execute(Index);
					}

					return FText::AsNumber(Index);
				}))
				.Font(This->FontAttr);
		});
		BaseArgs.OnSelectionChanged_Lambda([WeakThis = AsWeak()](TSharedPtr<int32> Item, ESelectInfo::Type SelectType)
		{
			TSharedPtr<SIndexComboBox> This = StaticCastSharedPtr<SIndexComboBox>(WeakThis.Pin());
			if (!This.IsValid())
			{
				return;
			}
			
			if (!Item.IsValid())
			{
				return;
			}
			
			This->OnIndexChanged.ExecuteIfBound(*Item, SelectType);
		});
		BaseArgs.Content()
		[
			SNew(STextBlock)
			.Text(TAttribute<FText>::CreateLambda([WeakThis = AsWeak()]
			{
				TSharedPtr<SIndexComboBox> This = StaticCastSharedPtr<SIndexComboBox>(WeakThis.Pin());
				if (!This.IsValid())
				{
					return FText::GetEmpty();
				}
				
				if (This->Indices.IsEmpty())
				{
					return LOCTEXT("NoneLabel", "None");
				}
				
				const int32 SelectedIndex = This->SelectedIndexAttr.Get(INDEX_NONE);
				if (This->GetIndexDisplayText.IsBound())
				{
					return This->GetIndexDisplayText.Execute(SelectedIndex);
				}

				return FText::AsNumber(SelectedIndex);
			}))
			.Font(FontAttr)
		];

		SComboBox<TSharedPtr<int32>>::Construct(BaseArgs);

		FillIndexList();
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		SComboBox<TSharedPtr<int32>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		const int32 StartingIndex = StartingIndexAttr.Get(0);
        const int32 NumIndices = NumIndicesAttr.Get(0);

		if (Indices.Num() != NumIndices || (Indices.IsValidIndex(0) && *Indices[0] != StartingIndex))
		{
			FillIndexList();
		}

		if (Indices.Num() > 0)
		{
			const int32 DesiredSelection = SelectedIndexAttr.Get(INDEX_NONE);
			const int32 CurrentSelection = GetSelectedItem().IsValid() ? *GetSelectedItem() : INDEX_NONE;
			if (DesiredSelection != CurrentSelection)
			{
				TSharedPtr<int32>* ItemToSelect = Indices.FindByPredicate([DesiredSelection](const TSharedPtr<int32>& Item)
				{
					return Item.IsValid() && *Item == DesiredSelection;
				});

				if (ItemToSelect)
				{
					SetSelectedItem(*ItemToSelect);
				}
				else
				{
					ClearSelection();
				}
			}
		}
		else
		{
			ClearSelection();
		}
	}
	
private:
	void FillIndexList()
	{
		Indices.Empty();

		const int32 StartingIndex = StartingIndexAttr.Get(0);
		const int32 NumIndices = NumIndicesAttr.Get(0);
		for (int32 Index = 0; Index < NumIndices; ++Index)
		{
			Indices.Add(MakeShared<int32>(Index + StartingIndex));
		}

		RefreshOptions();
		
		const int32 IndexToSelect = SelectedIndexAttr.Get(INDEX_NONE);

		TSharedPtr<int32>* ItemToSelect = Indices.FindByPredicate([IndexToSelect](const TSharedPtr<int32>& Item)
		{
			return Item.IsValid() && *Item == IndexToSelect;
		});

		if (ItemToSelect)
		{
			SetSelectedItem(*ItemToSelect);
		}
		else
		{
			ClearSelection();
		}
	}
	
private:
	TArray<TSharedPtr<int32>> Indices;

	TAttribute<int32> NumIndicesAttr;
	TAttribute<int32> StartingIndexAttr;
	TAttribute<int32> SelectedIndexAttr;
	TAttribute<FSlateFontInfo> FontAttr;
	
	FGetIndexDisplayText GetIndexDisplayText;
	FOnIndexChanged OnIndexChanged;
};

void FMediaProfileMediaSourceDetailCustomization::AddMediaPlayerProperties(UMediaSource* InMediaSource, IDetailLayoutBuilder& DetailBuilder)
{
	UMediaProfile* ParentProfile = InMediaSource->GetTypedOuter<UMediaProfile>();
	if (!ParentProfile)
	{
		return;
	}

	UMediaTexture* MediaTexture = ParentProfile->GetPlaybackManager()->GetSourceMediaTexture(InMediaSource);
	if (!MediaTexture)
	{
		return;
	}

	SourceMediaPlayer = MediaTexture->GetMediaPlayer();
	if (!SourceMediaPlayer.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = SourceMediaPlayer.Pin();
	
	IDetailCategoryBuilder& PlaybackOptionsCategory = DetailBuilder.EditCategory(TEXT("PlaybackOptionsCategory"), LOCTEXT("PlaybackOptionsCategoryName", "Playback Options"));

	const EMediaPlayerTrack TrackTypes[] =
	{
		EMediaPlayerTrack::Audio,
		EMediaPlayerTrack::Caption,
		EMediaPlayerTrack::Subtitle,
		EMediaPlayerTrack::Text,
		EMediaPlayerTrack::Video
	};
	
	constexpr bool bForAdvanced = false;
	constexpr bool bStartExpanded = true;
	
	IDetailGroup& TracksGroup = PlaybackOptionsCategory.AddGroup(TEXT("TracksGroup"), LOCTEXT("TracksGroupName", "Tracks"), bForAdvanced, bStartExpanded);

	for (EMediaPlayerTrack TrackType : TrackTypes)
	{
		TracksGroup.AddWidgetRow()
			.NameContent()
			[
				SNew(STextBlock)
				.Text(StaticEnum<EMediaPlayerTrack>()->GetDisplayNameTextByIndex((int32)TrackType))
				.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SIndexComboBox)
				.NumIndices(TAttribute<int32>::CreateLambda([WeakThis = AsWeak(), TrackType]()->int32
				{
					TSharedPtr<FMediaProfileMediaSourceDetailCustomization> This =
						StaticCastSharedPtr<FMediaProfileMediaSourceDetailCustomization>(WeakThis.Pin());

					if (!This.IsValid())
					{
						return 0;
					}
					
					if (TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = This->SourceMediaPlayer.Pin())
					{
						const int32 NumTracks = PinnedMediaPlayer->GetNumTracks(TrackType);
						return NumTracks > 0 ? NumTracks + 1 : 0;
					}

					return 0;
				}))
				.StartingIndex(INDEX_NONE)
				.SelectedIndex_Lambda([WeakThis = AsWeak(), TrackType]()->int32
				{
					TSharedPtr<FMediaProfileMediaSourceDetailCustomization> This =
						StaticCastSharedPtr<FMediaProfileMediaSourceDetailCustomization>(WeakThis.Pin());

					if (!This.IsValid())
					{
						return INDEX_NONE;
					}
					
					if (TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = This->SourceMediaPlayer.Pin())
					{
						return PinnedMediaPlayer->GetSelectedTrack(TrackType);
					}

					return INDEX_NONE;
				})
				.GetIndexDisplayText_Lambda([WeakThis = AsWeak(), TrackType](int32 TrackIndex)
				{
					TSharedPtr<FMediaProfileMediaSourceDetailCustomization> This =
						StaticCastSharedPtr<FMediaProfileMediaSourceDetailCustomization>(WeakThis.Pin());

					if (!This.IsValid())
					{
						return FText::GetEmpty();
					}
					
					if (TrackIndex == INDEX_NONE)
					{
						return LOCTEXT("DisabledLabel", "Disabled");
					}
					
					if (TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = This->SourceMediaPlayer.Pin())
					{
						FInternationalization& I18n = FInternationalization::Get();
						
						const FText DisplayName = PinnedMediaPlayer->GetTrackDisplayName(TrackType, TrackIndex);
						const FString Language = PinnedMediaPlayer->GetTrackLanguage(TrackType, TrackIndex);
						const FCulturePtr Culture = I18n.GetCulture(Language);
						const FString LanguageDisplayName = Culture.IsValid() ? Culture->GetDisplayName() : FString();
						const FString LanguageNativeName = Culture.IsValid() ? Culture->GetNativeName() : FString();
			
						return LanguageNativeName.IsEmpty() ? DisplayName : FText::Format(LOCTEXT("TrackNameFormat", "{0} ({1})"), DisplayName, FText::FromString(LanguageNativeName));
					}

					return LOCTEXT("NoneLabel", "None");
				})
				.OnIndexChanged_Lambda([WeakThis = AsWeak(), TrackType](int32 TrackIndex, ESelectInfo::Type SelectType)
				{
					TSharedPtr<FMediaProfileMediaSourceDetailCustomization> This =
						StaticCastSharedPtr<FMediaProfileMediaSourceDetailCustomization>(WeakThis.Pin());

					if (!This.IsValid())
					{
						return;
					}
					
					if (TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = This->SourceMediaPlayer.Pin())
					{
						PinnedMediaPlayer->SelectTrack(TrackType, TrackIndex);
					}
				})
			];
	}
	
	IDetailGroup& FormatsGroup = PlaybackOptionsCategory.AddGroup(TEXT("FormatsGroup"), LOCTEXT("FormatsGroupName", "Formats"), bForAdvanced, bStartExpanded);

	for (EMediaPlayerTrack TrackType : TrackTypes)
	{
		FormatsGroup.AddWidgetRow()
			.NameContent()
			[
				SNew(STextBlock)
				.Text(StaticEnum<EMediaPlayerTrack>()->GetDisplayNameTextByIndex((int32)TrackType))
				.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SIndexComboBox)
				.NumIndices(TAttribute<int32>::CreateLambda([WeakThis = AsWeak(), TrackType]()->int32
				{
					TSharedPtr<FMediaProfileMediaSourceDetailCustomization> This =
						StaticCastSharedPtr<FMediaProfileMediaSourceDetailCustomization>(WeakThis.Pin());

					if (!This.IsValid())
					{
						return 0;
					}
					
					if (TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = This->SourceMediaPlayer.Pin())
					{
						const int32 SelectedTrack = PinnedMediaPlayer->GetSelectedTrack(TrackType);
						const int32 NumFormats = PinnedMediaPlayer->GetNumTrackFormats(TrackType, SelectedTrack);
						return SelectedTrack != INDEX_NONE ? NumFormats : 0;
					}

					return 0;
				}))
				.SelectedIndex_Lambda([WeakThis = AsWeak(), TrackType]()->int32
				{
					TSharedPtr<FMediaProfileMediaSourceDetailCustomization> This =
						StaticCastSharedPtr<FMediaProfileMediaSourceDetailCustomization>(WeakThis.Pin());

					if (!This.IsValid())
					{
						return INDEX_NONE;
					}
					
					if (TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = This->SourceMediaPlayer.Pin())
					{
						const int32 SelectedTrack = PinnedMediaPlayer->GetSelectedTrack(TrackType);
						return SelectedTrack != INDEX_NONE ? PinnedMediaPlayer->GetTrackFormat(TrackType, SelectedTrack) : INDEX_NONE;
					}

					return INDEX_NONE;
				})
				.GetIndexDisplayText_Lambda([WeakThis = AsWeak(), TrackType](int32 FormatIndex)
				{
					TSharedPtr<FMediaProfileMediaSourceDetailCustomization> This =
						StaticCastSharedPtr<FMediaProfileMediaSourceDetailCustomization>(WeakThis.Pin());

					if (!This.IsValid())
					{
						return FText::GetEmpty();
					}
					
					if (FormatIndex == INDEX_NONE)
					{
						return LOCTEXT("DisabledLabel", "Disabled");
					}
					
					if (TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = This->SourceMediaPlayer.Pin())
					{
						const int32 SelectedTrack = PinnedMediaPlayer->GetSelectedTrack(TrackType);
						if (SelectedTrack == INDEX_NONE)
						{
							return LOCTEXT("NoneLabel", "None");
						}
						
						FText DisplayText;
						
						if (TrackType == EMediaPlayerTrack::Audio)
						{
							const uint32 Channels = PinnedMediaPlayer->GetAudioTrackChannels(SelectedTrack, FormatIndex);
							const uint32 SampleRate = PinnedMediaPlayer->GetAudioTrackSampleRate(SelectedTrack, FormatIndex);
							const FString Type = PinnedMediaPlayer->GetAudioTrackType(SelectedTrack, FormatIndex);

							DisplayText = FText::Format(LOCTEXT("TrackFormatMenuAudioFormat", "{0}: {1} {2} channels @ {3} Hz"),
								FText::AsNumber(FormatIndex),
								FText::FromString(Type),
								FText::AsNumber(Channels),
								FText::AsNumber(SampleRate)
							);
						}
						else if (TrackType == EMediaPlayerTrack::Video)
						{
							const FIntPoint Dim = PinnedMediaPlayer->GetVideoTrackDimensions(SelectedTrack, FormatIndex);
							const float FrameRate = PinnedMediaPlayer->GetVideoTrackFrameRate(SelectedTrack, FormatIndex);
							const TRange<float> FrameRates = PinnedMediaPlayer->GetVideoTrackFrameRates(SelectedTrack, FormatIndex);
							const FString Type = PinnedMediaPlayer->GetVideoTrackType(SelectedTrack, FormatIndex);

							FFormatNamedArguments Arguments;
							Arguments.Add(TEXT("Index"), FText::AsNumber(FormatIndex));
							Arguments.Add(TEXT("DimX"), FText::AsNumber(Dim.X));
							Arguments.Add(TEXT("DimY"), FText::AsNumber(Dim.Y));
							Arguments.Add(TEXT("Fps"), FText::AsNumber(FrameRate));
							Arguments.Add(TEXT("Type"), FText::FromString(Type));

							if (FrameRates.IsDegenerate() && (FrameRates.GetLowerBoundValue() == FrameRate))
							{
								DisplayText = FText::Format(LOCTEXT("TrackFormatMenuVideoFormat", "{Index}: {Type} {DimX}x{DimY} {Fps} fps"), Arguments);
							}
							else
							{
								Arguments.Add(TEXT("FpsLower"), FText::AsNumber(FrameRates.GetLowerBoundValue()));
								Arguments.Add(TEXT("FpsUpper"), FText::AsNumber(FrameRates.GetUpperBoundValue()));

								DisplayText = FText::Format(LOCTEXT("TrackFormatMenuVideoFormat2", "{Index}: {Type} {DimX}x{DimY} {Fps} [{FpsLower}-{FpsUpper}] fps"), Arguments);
							}
						}
						else
						{
							DisplayText = LOCTEXT("TrackFormatDefault", "Default");
						}

						return DisplayText;
					}
					
					return LOCTEXT("NoneLabel", "None");
				})
				.OnIndexChanged_Lambda([WeakThis = AsWeak(), TrackType](int32 FormatIndex, ESelectInfo::Type SelectType)
				{
					TSharedPtr<FMediaProfileMediaSourceDetailCustomization> This =
						StaticCastSharedPtr<FMediaProfileMediaSourceDetailCustomization>(WeakThis.Pin());

					if (!This.IsValid())
					{
						return;
					}
					
					if (TStrongObjectPtr<UMediaPlayer> PinnedMediaPlayer = This->SourceMediaPlayer.Pin())
					{
						const int32 SelectedTrack = PinnedMediaPlayer->GetSelectedTrack(TrackType);
						if (SelectedTrack != INDEX_NONE)
						{
							PinnedMediaPlayer->SetTrackFormat(TrackType, SelectedTrack, FormatIndex);
						}
					}
				})
			];
	}
}

class SMediaCaptureMethodComboButton : public SComboButton
{
public:
	SLATE_BEGIN_ARGS(SMediaCaptureMethodComboButton) { }
		SLATE_EVENT(FSimpleDelegate, OnMethodChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TArray<TWeakObjectPtr<UObject>>& InObjects)
	{
		OnMethodChanged = InArgs._OnMethodChanged;
		Objects = InObjects;
		
		SComboButton::Construct(SComboButton::FArguments()
			.OnGetMenuContent_Lambda([this]
			{
				FMenuBuilder MenuBuilder(true, nullptr);

				MenuBuilder.AddMenuEntry(LOCTEXT("MediaCaptureCurrentViewportMethod", "Current Viewport"),
					LOCTEXT("MediaCaptureCurrentViewportMethodToolTip", "Capture from the current viewport"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SMediaCaptureMethodComboButton::OnCurrentViewportMethodSelected),
						FCanExecuteAction::CreateStatic(&MediaProfilePlaybackManager::IsActiveViewportCaptureAllowed)));
				
				MenuBuilder.AddMenuEntry(LOCTEXT("MediaCaptureViewportMethod", "Media Viewport"),
					LOCTEXT("MediaCaptureViewportMethodToolTip", "Capture from a viewport"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &SMediaCaptureMethodComboButton::OnViewportMethodSelected),
						FCanExecuteAction::CreateStatic(&MediaProfilePlaybackManager::IsManagedViewportCaptureAllowed)));

				MenuBuilder.AddMenuEntry(LOCTEXT("MediaCaptureRenderTargetMethod", "Render Target"),
					LOCTEXT("MediaCaptureRenderTargetMethodToolTip", "Capture from a render target"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SMediaCaptureMethodComboButton::OnRenderTargetMethodSelected)));

				MenuBuilder.AddMenuEntry(LOCTEXT("MediaCaptureMediaTextureMethod", "Media Texture"),
					LOCTEXT("MediaCaptureMediaTextureMethodToolTip", "Capture from a media texture (passthrough)"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SMediaCaptureMethodComboButton::OnMediaTextureMethodSelected)));
				
				return MenuBuilder.MakeWidget();
			})
			.ContentPadding(0.f)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SMediaCaptureMethodComboButton::GetButtonText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]);
	}

private:
	void OnCurrentViewportMethodSelected()
	{
		UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
		if (!MediaCaptureSettings)
		{
			return;
		}
		
		for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
		{
			TStrongObjectPtr<UObject> Object = WeakObject.Pin();
			UMediaOutput* MediaOutput = Object.IsValid() ? Cast<UMediaOutput>(Object.Get()) : nullptr;
			if (!MediaOutput)
			{
				continue;
			}

			if (MediaOutput->IsA<UDummyMediaOutput>())
			{
				continue;
			}
			
			// Now, if the object has an existing capture config, we want to copy that config over to the current viewport config, so find it
			MediaCaptureSettings->CurrentViewportMediaOutput.CaptureOptions = FindExistingMediaCaptureOptions(MediaOutput, MediaCaptureSettings, FMediaCaptureOptions());

			// We expect there to be only one capture configuration per media output, so clear
			// any existing capture configurations in the media capture settings before adding the new one
			RemoveExistingMediaCaptures(MediaOutput, MediaCaptureSettings);
			
			MediaCaptureSettings->CurrentViewportMediaOutput.MediaOutput = MediaOutput;
		}

		MediaCaptureSettings->SaveConfig();
		OnMethodChanged.ExecuteIfBound();
	}
	
	void OnViewportMethodSelected()
	{
		UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
		if (!MediaCaptureSettings)
		{
			return;
		}
		
		for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
		{
			TStrongObjectPtr<UObject> Object = WeakObject.Pin();
			UMediaOutput* MediaOutput = Object.IsValid() ? Cast<UMediaOutput>(Object.Get()) : nullptr;
			if (!MediaOutput)
			{
				continue;
			}

			if (MediaOutput->IsA<UDummyMediaOutput>())
			{
				continue;
			}
			
			// We expect there to be only one capture configuration per media output, so clear
			// any existing viewport capture configurations before adding the new one
			UE::MediaFrameworkWorldSettings::Helpers::RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(MediaCaptureSettings, MediaOutput);

			FMediaFrameworkCaptureCameraViewportCameraOutputInfo NewViewportOutputInfo;
			NewViewportOutputInfo.MediaOutput = MediaOutput;
			
			// Now, if the object has an existing render target capture config, we want to copy that config over to the new viewport config, so find it
			NewViewportOutputInfo.CaptureOptions = FindExistingMediaCaptureOptions(MediaOutput, MediaCaptureSettings, NewViewportOutputInfo.CaptureOptions);

			RemoveExistingMediaCaptures(MediaOutput, MediaCaptureSettings);
			MediaCaptureSettings->ViewportCaptures.Add(NewViewportOutputInfo);
		}

		MediaCaptureSettings->SaveConfig();
		OnMethodChanged.ExecuteIfBound();
	}

	void OnRenderTargetMethodSelected()
	{
		UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
		if (!MediaCaptureSettings)
		{
			return;
		}
		
		for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
		{
			TStrongObjectPtr<UObject> Object = WeakObject.Pin();
			UMediaOutput* MediaOutput = Object.IsValid() ? Cast<UMediaOutput>(Object.Get()) : nullptr;
			if (!MediaOutput)
			{
				continue;
			}

			if (MediaOutput->IsA<UDummyMediaOutput>())
			{
				continue;
			}

			// We expect there to be only one capture configuration per media output, so clear
			// any existing render target capture configurations before adding the new one
			UE::MediaFrameworkWorldSettings::Helpers::RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(MediaCaptureSettings, MediaOutput);

			FMediaFrameworkCaptureRenderTargetCameraOutputInfo NewRenderTargetOutputInfo;
			NewRenderTargetOutputInfo.MediaOutput = MediaOutput;
			
			// Now, if the object has an existing render target capture config, we want to copy that config over to the new viewport config, so find it
			NewRenderTargetOutputInfo.CaptureOptions = FindExistingMediaCaptureOptions(MediaOutput, MediaCaptureSettings, NewRenderTargetOutputInfo.CaptureOptions);

			RemoveExistingMediaCaptures(MediaOutput, MediaCaptureSettings);
			MediaCaptureSettings->RenderTargetCaptures.Add(NewRenderTargetOutputInfo);
		}

		MediaCaptureSettings->SaveConfig();
		OnMethodChanged.ExecuteIfBound();
	}

	void OnMediaTextureMethodSelected()
	{
		UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
		if (!MediaCaptureSettings)
		{
			return;
		}
		
		for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
		{
			TStrongObjectPtr<UObject> Object = WeakObject.Pin();
			UMediaOutput* MediaOutput = Object.IsValid() ? Cast<UMediaOutput>(Object.Get()) : nullptr;
			if (!MediaOutput)
			{
				continue;
			}

			if (MediaOutput->IsA<UDummyMediaOutput>())
			{
				continue;
			}

			// We expect there to be only one capture configuration per media output, so clear
			// any existing render target capture configurations before adding the new one
			UE::MediaFrameworkWorldSettings::Helpers::RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(MediaCaptureSettings, MediaOutput);

			FMediaFrameworkCaptureMediaTextureOutputInfo NewMediaTextureOutputInfo;
			NewMediaTextureOutputInfo.MediaOutput = MediaOutput;
			
			// Unlike other methods, don't copy capture options from other capture types to media texture, as the there are a specific
			// set of default options for media textures that are the ideal settings.
			
			RemoveExistingMediaCaptures(MediaOutput, MediaCaptureSettings);
			MediaCaptureSettings->MediaTextureCaptures.Add(NewMediaTextureOutputInfo);
		}

		MediaCaptureSettings->SaveConfig();
		OnMethodChanged.ExecuteIfBound();
	}

	FText GetButtonText() const
	{
		UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
		if (!MediaCaptureSettings)
		{
			return LOCTEXT("MediaCaptureMethodNotConfigured", "Not Configured");
		}
		
		int32 NumCurrentViewports = 0;
		int32 NumViewports = 0;
		int32 NumRenderTargets = 0;
		int32 NumMediaTextures = 0;

		for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
		{
			TStrongObjectPtr<UObject> Object = WeakObject.Pin();
			UMediaOutput* MediaOutput = Object.IsValid() ? Cast<UMediaOutput>(Object.Get()) : nullptr;
			if (!MediaOutput)
			{
				continue;
			}

			using namespace UE::MediaFrameworkWorldSettings::Helpers;
			NumCurrentViewports += FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(MediaCaptureSettings, MediaOutput).Num();
			NumViewports += FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(MediaCaptureSettings, MediaOutput).Num();
			NumRenderTargets += FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(MediaCaptureSettings, MediaOutput).Num();
			NumMediaTextures += FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(MediaCaptureSettings, MediaOutput).Num();
		}

		const int32 NumTypesWithOutputInfo = (int32)(NumCurrentViewports > 0) + (int32)(NumViewports > 0) + (int32)(NumRenderTargets > 0) + (int32)(NumMediaTextures > 0);
		if (NumTypesWithOutputInfo >= 2)
		{
			return LOCTEXT("MediaCaptureMethodMultipleValues", "Multiple Values");
		}
		if (NumCurrentViewports > 0)
		{
			return LOCTEXT("MediaCaptureCurrentViewportMethod", "Current Viewport");
		}
		if (NumViewports > 0)
		{
			return LOCTEXT("MediaCaptureViewportMethod", "Media Viewport");
		}
		if (NumRenderTargets > 0)
		{
			return LOCTEXT("MediaCaptureRenderTargetMethod", "Render Target");
		}
		if (NumMediaTextures > 0)
		{
			return LOCTEXT("MediaCaptureMediaTextureMethod", "Media Texture");
		}

		return LOCTEXT("MediaCaptureMethodNotConfigured", "Not Configured");
	}
	
	FMediaCaptureOptions FindExistingMediaCaptureOptions(UMediaOutput* InMediaOutput, UMediaProfileEditorCaptureSettings* MediaCaptureSettings, const FMediaCaptureOptions& DefaultOptions) const
	{
		using namespace UE::MediaFrameworkWorldSettings::Helpers;
		
		if (FMediaFrameworkCaptureCurrentViewportOutputInfo* CurrentViewportCapture =
			FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(MediaCaptureSettings, InMediaOutput))
		{
			return CurrentViewportCapture->CaptureOptions;
		}
		
		if (FMediaFrameworkCaptureCameraViewportCameraOutputInfo* ViewportCapture =
			FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(MediaCaptureSettings, InMediaOutput))
		{
			return ViewportCapture->CaptureOptions;
		}
		
		if (FMediaFrameworkCaptureRenderTargetCameraOutputInfo* RenderTargetCapture =
			FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(MediaCaptureSettings, InMediaOutput))
		{
			return RenderTargetCapture->CaptureOptions;
		}
		
		// We don't transfer options from the media texture capture method to other methods as many of the capture options and defaults different for media textures.
		// If no capture options are found, return the default options
		return DefaultOptions;
	}

	void RemoveExistingMediaCaptures(UMediaOutput* InMediaOutput, UMediaProfileEditorCaptureSettings* MediaCaptureSettings)
	{
		using namespace UE::MediaFrameworkWorldSettings::Helpers;
		
		RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(MediaCaptureSettings, InMediaOutput);
		RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(MediaCaptureSettings, InMediaOutput);
		RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(MediaCaptureSettings, InMediaOutput);
		RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(MediaCaptureSettings, InMediaOutput);
	}
	
private:
	TArray<TWeakObjectPtr<UObject>> Objects;
	FSimpleDelegate OnMethodChanged;
};

class FMediaFrameworkBaseOutputInfoCustomization : public IPropertyTypeCustomization
{
public:
	FMediaFrameworkBaseOutputInfoCustomization(const TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor)
		: MediaProfileEditor(InMediaProfileEditor)
	{ }
	
protected:
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override { }

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		TMap<FName, TFunction<void(TSharedPtr<IPropertyHandle>)>> PropertyCustomizers =
		{
			{
				GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, ResizeMethod),
				[this, &ChildBuilder](TSharedPtr<IPropertyHandle> PropertyHandle)
				{
					FPropertyComboBoxArgs ComboBoxArgs;
					ComboBoxArgs.PropertyHandle = PropertyHandle;
					ComboBoxArgs.OnGetStrings = FOnGetPropertyComboBoxStrings::CreateSP(this, &FMediaFrameworkBaseOutputInfoCustomization::GetResizeMethodComboBoxStrings);
					ComboBoxArgs.OnGetValue = FOnGetPropertyComboBoxValue::CreateSP(this, &FMediaFrameworkBaseOutputInfoCustomization::GetResizeMethodComboBoxValue, PropertyHandle);
					ComboBoxArgs.OnValueSelected = FOnPropertyComboBoxValueSelected::CreateSP(this, &FMediaFrameworkBaseOutputInfoCustomization::SetResizeMethodValue, PropertyHandle);
					
					ChildBuilder.AddCustomRow(PropertyHandle->GetPropertyDisplayName())
					.PropertyHandleList({ PropertyHandle })
					.NameContent()
					[
						PropertyHandle->CreatePropertyNameWidget()
					]
					.ValueContent()
					[
						PropertyCustomizationHelpers::MakePropertyComboBox(ComboBoxArgs)
					];
				}
			}
		};
		
		// Manually add all capture options properties to avoid an additional property group getting added
		TSharedPtr<IPropertyHandle> CaptureOptionsHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaFrameworkCaptureCurrentViewportOutputInfo, CaptureOptions));
		uint32 NumChildren;
		CaptureOptionsHandle->GetNumChildren(NumChildren);
		for (uint32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = CaptureOptionsHandle->GetChildHandle(Index);
			ChildHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMediaFrameworkBaseOutputInfoCustomization::PropertyValueChange));
			
			const FName ChildPropertyName = ChildHandle->GetProperty()->GetFName();
			if (PropertyCustomizers.Contains(ChildPropertyName))
			{
				PropertyCustomizers[ChildPropertyName](ChildHandle);
			}
			else
			{
				ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
			}
		}
	}
	
	/** Raised when any of the capture info properties have changed */
	virtual void PropertyValueChange(const FPropertyChangedEvent& PropertyChangedEvent)
	{
		if (!bBroadcastPropertyChanged)
		{
			return;
		}
		
		if (UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			MediaCaptureSettings->SaveConfig();
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(MediaCaptureSettings, const_cast<FPropertyChangedEvent&>(PropertyChangedEvent));
		}
	}

	/** Gets the EMediaCaptureResizeMethod display strings for the ResizeMethod combo box */
	void GetResizeMethodComboBoxStrings(TArray<TSharedPtr<FString>>& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems)
	{
		const UEnum* ResizeMethodEnum = StaticEnum<EMediaCaptureResizeMethod>();

		const TArray<EMediaCaptureResizeMethod> RestrictedResizeMethods = GetRestrictedResizeMethods();
		
		const int32 NumEnums = ResizeMethodEnum->NumEnums() - 1;
		for (int32 Entry = 0; Entry < NumEnums; ++Entry)
		{
			FText EnumName = ResizeMethodEnum->GetDisplayNameTextByIndex(Entry);

			OutStrings.Add(MakeShared<FString>(EnumName.ToString()));
			OutToolTips.Add(SNew(SToolTip).Text(ResizeMethodEnum->GetToolTipTextByIndex(Entry)));
			OutRestrictedItems.Add(RestrictedResizeMethods.Contains((EMediaCaptureResizeMethod)Entry));
		}
	}

	/** Gets the current value EMediaCaptureResizeMethod display string for the ResizeMethod combo box */
	FString GetResizeMethodComboBoxValue(TSharedPtr<IPropertyHandle> PropertyHandle)
	{
		const UEnum* ResizeMethodEnum = StaticEnum<EMediaCaptureResizeMethod>();
		
		uint8 Value;
		if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
		{
			return ResizeMethodEnum->GetDisplayNameTextByIndex(Value).ToString();
		}
		
		return TEXT("Failed to get value");
	}
	
	/** Sets the ResizeMethod value from the selected option from the combo box*/
	void SetResizeMethodValue(const FString& InValue, TSharedPtr<IPropertyHandle> PropertyHandle)
	{
		uint8 Value;
		PropertyHandle->GetValue(Value);
		
		const UEnum* ResizeMethodEnum = StaticEnum<EMediaCaptureResizeMethod>();
		int32 EnumIndex = INDEX_NONE;
		const int32 NumEnums = ResizeMethodEnum->NumEnums() - 1;
		for (int32 Entry = 0; Entry < NumEnums; ++Entry)
		{
			FText EnumName = ResizeMethodEnum->GetDisplayNameTextByIndex(Entry);

			if (EnumName.ToString() == InValue)
			{
				EnumIndex = Entry;
				break;
			}
		}
		
		if (Value != (uint8)EnumIndex)
		{
			PropertyHandle->SetValue((uint8)EnumIndex);
		}
	}
	
	/** Gets the list of EMediaCaptureResizeMethod options that should be restricted for the current customization */
	virtual TArray<EMediaCaptureResizeMethod> GetRestrictedResizeMethods() const
	{
		return TArray<EMediaCaptureResizeMethod>();
	}
	
protected:
	/** Weak pointer to the  media profile editor */
	TWeakPtr<FMediaProfileEditor> MediaProfileEditor;
	
	/** Indicates that ObjectPropertyChanged event should be broadcast when a child property value has changed */
	bool bBroadcastPropertyChanged = true;
};

class FMediaFrameworkViewportOutputInfoCustomization : public FMediaFrameworkBaseOutputInfoCustomization
{
public:
	FMediaFrameworkViewportOutputInfoCustomization(const TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor)
		: FMediaFrameworkBaseOutputInfoCustomization(InMediaProfileEditor)
	{ }

protected:
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		TSharedPtr<IPropertyHandle> LockedActorsHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaFrameworkCaptureCameraViewportCameraOutputInfo, Cameras));
		LockedActorsHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMediaFrameworkViewportOutputInfoCustomization::PropertyValueChange));
		LockedActorsHandle->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMediaFrameworkViewportOutputInfoCustomization::PropertyValueChange));
		ChildBuilder.AddProperty(LockedActorsHandle.ToSharedRef());

		FMediaFrameworkBaseOutputInfoCustomization::CustomizeChildren(PropertyHandle, ChildBuilder, CustomizationUtils);
	}
};

class FMediaFrameworkRenderTargetOutputInfoCustomization : public FMediaFrameworkBaseOutputInfoCustomization
{
public:
	FMediaFrameworkRenderTargetOutputInfoCustomization(const TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor)
		: FMediaFrameworkBaseOutputInfoCustomization(InMediaProfileEditor)
	{ }

protected:
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		TSharedPtr<IPropertyHandle> RenderTargetHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaFrameworkCaptureRenderTargetCameraOutputInfo, RenderTarget));
		RenderTargetHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMediaFrameworkRenderTargetOutputInfoCustomization::PropertyValueChange));
		ChildBuilder.AddProperty(RenderTargetHandle.ToSharedRef());

		FMediaFrameworkBaseOutputInfoCustomization::CustomizeChildren(PropertyHandle, ChildBuilder, CustomizationUtils);
	}
};

class FMediaFrameworkMediaTextureOutputInfoCustomization : public FMediaFrameworkBaseOutputInfoCustomization
{
public:
	FMediaFrameworkMediaTextureOutputInfoCustomization(const TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor)
		: FMediaFrameworkBaseOutputInfoCustomization(InMediaProfileEditor)
	{ }

protected:
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		TSharedPtr<IPropertyHandle> MediaTextureHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaFrameworkCaptureMediaTextureOutputInfo, MediaTexture));
		MediaTextureHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::PropertyValueChange));

		ChildBuilder.AddCustomRow(MediaTextureHandle->GetPropertyDisplayName())
		.NameContent()
		[
			MediaTextureHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SMediaProfileSourceTexturePicker)
			.TexturePropertyHandle(MediaTextureHandle)
			.ThumbnailPool(CustomizationUtils.GetThumbnailPool())
			.OnMediaSourceSelected(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::OnMediaSourceSelected)
		];

		ResizeMethodProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, ResizeMethod));
		RotationProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaCaptureTransform, Rotation));
		RotationProperty->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::PropertyValueChange));
		
		ChildBuilder.AddCustomRow(LOCTEXT("RotationFilterString", "Rotation"))
			.PropertyHandleList({ RotationProperty })
			.NameContent()
			[
				RotationProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				CreateRotationWidget()
			];
	
		FlipHorizontalProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaCaptureTransform, bFlipHorizontal));
		FlipHorizontalProperty->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::PropertyValueChange));
		
		FlipVerticalProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaCaptureTransform, bFlipVertical));
		FlipVerticalProperty->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::PropertyValueChange));
		
		ChildBuilder.AddCustomRow(LOCTEXT("FlipFilterString", "Flip"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FlipName", "Flip"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				CreateFlipWidget()
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
				TAttribute<bool>::CreateSP(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::CanResetFlipToDefault),
				FSimpleDelegate::CreateSP(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::ResetFlipToDefault)
			));

		FMediaFrameworkBaseOutputInfoCustomization::CustomizeChildren(PropertyHandle, ChildBuilder, CustomizationUtils);
	}

	virtual TArray<EMediaCaptureResizeMethod> GetRestrictedResizeMethods() const override
	{
		TArray<EMediaCaptureResizeMethod> DisallowedResizeMethods = { EMediaCaptureResizeMethod::ResizeSource };
		
		if (RotationProperty.IsValid() && RotationProperty->IsValidHandle())
		{
			uint8 RotationValue;
			if (RotationProperty->GetValue(RotationValue) == FPropertyAccess::Success)
			{
				if ((EMediaCaptureRotation)RotationValue != EMediaCaptureRotation::None)
				{
					DisallowedResizeMethods.Add(EMediaCaptureResizeMethod::None);
				}
			}
		}
		
		return DisallowedResizeMethods;
	}
	
private:
	/** Raised when a media source is selected from the media profile media source options dropdown */
	void OnMediaSourceSelected(UMediaProfile* InMediaProfile, int32 InMediaSourceIndex)
	{
		if (InMediaProfile == nullptr)
		{
			return;
		}
		
		if (TSharedPtr<FMediaProfileEditor> PinnedMediaProfileEditor = MediaProfileEditor.Pin())
		{
			InMediaProfile->GetPlaybackManager()->OpenSourceFromIndex(InMediaSourceIndex, PinnedMediaProfileEditor.Get());
		}
	}
	
	TSharedRef<SWidget> CreateRotationWidget()
	{
		return SNew(SSegmentedControl<EMediaCaptureRotation>)
			.OnValueChanged(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::SetMediaCaptureRotation)
			.Value(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::GetMediaCaptureRotation)
			
			+SSegmentedControl<EMediaCaptureRotation>::Slot(EMediaCaptureRotation::CCW90)
			.Text(StaticEnum<EMediaCaptureRotation>()->GetDisplayNameTextByValue((int64)EMediaCaptureRotation::CCW90))
			
			+SSegmentedControl<EMediaCaptureRotation>::Slot(EMediaCaptureRotation::None)
			.Text(StaticEnum<EMediaCaptureRotation>()->GetDisplayNameTextByValue((int64)EMediaCaptureRotation::None))
			
			+SSegmentedControl<EMediaCaptureRotation>::Slot(EMediaCaptureRotation::CW90)
			.Text(StaticEnum<EMediaCaptureRotation>()->GetDisplayNameTextByValue((int64)EMediaCaptureRotation::CW90))
			
			+SSegmentedControl<EMediaCaptureRotation>::Slot(EMediaCaptureRotation::CW180)
			.Text(StaticEnum<EMediaCaptureRotation>()->GetDisplayNameTextByValue((int64)EMediaCaptureRotation::CW180));
	}

	EMediaCaptureRotation GetMediaCaptureRotation() const
	{
		if (!RotationProperty.IsValid() || !RotationProperty->IsValidHandle())
		{
			return EMediaCaptureRotation::None;
		}
		
		uint8 EnumValue;
		FPropertyAccess::Result Result = RotationProperty->GetValue(EnumValue);
		if (RotationProperty->GetValue(EnumValue) == FPropertyAccess::Success)
		{
			return (EMediaCaptureRotation)EnumValue;
		}
		
		return EMediaCaptureRotation::None;
	}

	void SetMediaCaptureRotation(EMediaCaptureRotation InNewRotation)
	{
		if (!RotationProperty.IsValid() || !RotationProperty->IsValidHandle())
		{
			return;
		}

		RotationProperty->SetValue((uint8)InNewRotation);
		
		if (InNewRotation != EMediaCaptureRotation::None)
		{
			if (ResizeMethodProperty.IsValid() && ResizeMethodProperty->IsValidHandle())
			{
				ResizeMethodProperty->SetValue((uint8)EMediaCaptureResizeMethod::ResizeInRenderPass);
			}
		}
	}

	TSharedRef<SWidget> CreateFlipWidget()
	{
		return SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.Padding(FMargin(6.0f, 4.0f))
				.IsChecked(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::IsFlipped, FlipHorizontalProperty)
				.OnCheckStateChanged(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::SetFlipped, FlipHorizontalProperty)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 2.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.FlipHorizontal"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FlipHorizontalLabel", "Flip Horizontal"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.Padding(FMargin(6.0f, 4.0f))
				.IsChecked(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::IsFlipped, FlipVerticalProperty)
				.OnCheckStateChanged(this, &FMediaFrameworkMediaTextureOutputInfoCustomization::SetFlipped, FlipVerticalProperty)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 2.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.FlipVertical"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FlipVerticalLabel", "Flip Vertical"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			];
	}

	ECheckBoxState IsFlipped(TSharedPtr<IPropertyHandle> InFlippedProperty) const
	{
		if (!InFlippedProperty.IsValid() || !InFlippedProperty->IsValidHandle())
		{
			return ECheckBoxState::Unchecked;
		}
		
		bool bFlippedValue;
		if (InFlippedProperty->GetValue(bFlippedValue) == FPropertyAccess::Success)
		{
			return bFlippedValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Unchecked;
	}

	void SetFlipped(ECheckBoxState CheckBoxState, TSharedPtr<IPropertyHandle> InFlippedProperty)
	{
		if (!InFlippedProperty.IsValid() || !InFlippedProperty->IsValidHandle())
		{
			return;
		}
		
		InFlippedProperty->SetValue(CheckBoxState == ECheckBoxState::Checked);
	}
	
	bool CanResetFlipToDefault() const
	{
		bool bFlipHorizontalValue = false;
		bool bFlipVerticalValue = false;
		
		if (FlipHorizontalProperty.IsValid() && FlipHorizontalProperty->IsValidHandle())
		{
			FlipHorizontalProperty->GetValue(bFlipHorizontalValue);
		}
		
		if (FlipVerticalProperty.IsValid() && FlipVerticalProperty->IsValidHandle())
		{
			FlipVerticalProperty->GetValue(bFlipVerticalValue);
		}
		
		return bFlipHorizontalValue || bFlipVerticalValue;
	}
	
	void ResetFlipToDefault()
	{
		{
			// Delay broadcasting the ObjectPropertyChanged event until both properties have been set
			TGuardValue<bool> Guard(bBroadcastPropertyChanged, false);
			if (FlipHorizontalProperty.IsValid() && FlipHorizontalProperty->IsValidHandle())
			{
				FlipHorizontalProperty->SetValue(false, EPropertyValueSetFlags::ResetToDefault);
			}
		
			if (FlipVerticalProperty.IsValid() && FlipVerticalProperty->IsValidHandle())
			{
				FlipVerticalProperty->SetValue(false, EPropertyValueSetFlags::ResetToDefault);
			}
		}

		FPropertyChangedEvent FlipHorizontalEvent(FindFProperty<FProperty>(FMediaCaptureTransform::StaticStruct(), GET_MEMBER_NAME_CHECKED(FMediaCaptureTransform, bFlipHorizontal)));
		PropertyValueChange(FlipHorizontalEvent);
		
		FPropertyChangedEvent FlipVerticalEvent(FindFProperty<FProperty>(FMediaCaptureTransform::StaticStruct(), GET_MEMBER_NAME_CHECKED(FMediaCaptureTransform, bFlipVertical)));
		PropertyValueChange(FlipVerticalEvent);
	}
	
private:
	TSharedPtr<IPropertyHandle> ResizeMethodProperty;
	TSharedPtr<IPropertyHandle> RotationProperty;
	TSharedPtr<IPropertyHandle> FlipHorizontalProperty;
	TSharedPtr<IPropertyHandle> FlipVerticalProperty;
};

FMediaProfileMediaOutputDetailCustomization::~FMediaProfileMediaOutputDetailCustomization()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void FMediaProfileMediaOutputDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FMediaProfileMediaItemDetailCustomization::CustomizeDetails(DetailBuilder);
	
	// Hack to avoid the details builder separating out categories that only have advanced properties in them and putting them at the bottom.
	// If there is any sort function provided, the builder lumps all categories together and sorts via their sort order. We don't need to manually
	// adjust the sort order, but we do need the details builder to lump all categories together, hence the empty sort function
	DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap) { });
	
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FMediaProfileMediaOutputDetailCustomization::OnObjectPropertyChanged);
	
	TArray<TWeakObjectPtr<UObject>> Objects;
	CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		FMediaFrameworkCaptureCurrentViewportOutputInfo::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([WeakMediaProfile = MediaProfileEditor]
		{
			return MakeShared<FMediaFrameworkBaseOutputInfoCustomization>(WeakMediaProfile);
		}));
	
	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		FMediaFrameworkCaptureCameraViewportCameraOutputInfo::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([WeakMediaProfile = MediaProfileEditor]
		{
			return MakeShared<FMediaFrameworkViewportOutputInfoCustomization>(WeakMediaProfile);
		}));

	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		FMediaFrameworkCaptureRenderTargetCameraOutputInfo::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([WeakMediaProfile = MediaProfileEditor]
		{
			return MakeShared<FMediaFrameworkRenderTargetOutputInfoCustomization>(WeakMediaProfile);
		}));

	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		FMediaFrameworkCaptureMediaTextureOutputInfo::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([WeakMediaProfile = MediaProfileEditor]
		{
			return MakeShared<FMediaFrameworkMediaTextureOutputInfoCustomization>(WeakMediaProfile);
		}));
	
	// For media outputs, add the corresponding capture info to the details panel
	const FName CaptureCategoryName = TEXT("MediaCaptureCategory");
	IDetailCategoryBuilder& MediaCaptureCategory = DetailBuilder.EditCategory(
		CaptureCategoryName,
		LOCTEXT("MediaCaptureCategoryLabel", "Capture"),
		ECategoryPriority::Important);

	RegisterCaptureSection(FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor"), CaptureCategoryName);
		
	MediaCaptureCategory
		.AddCustomRow(LOCTEXT("MediaCaptureMethodFilter", "Capture Method"))
        .Visibility(TAttribute<EVisibility>::CreateSP(this, &FMediaProfileMediaOutputDetailCustomization::GetValidObjectVisibility))
        .NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MediaCaptureMethodLabel", "Capture Method"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.f)
		.MaxDesiredWidth(250.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SMediaCaptureMethodComboButton, Objects)
				.OnMethodChanged(this, &FMediaProfileMediaOutputDetailCustomization::OnCaptureMethodChanged)
			]
		];

	TArray<TSharedPtr<FStructOnScope>> CaptureSettingStructs;
	bool bHasCurrentViewportCapture = false;
	bool bHasViewportCaptures = false;
	bool bHasRenderTargetCaptures = false;
	bool bHasMediaTextureCaptures = false;
	if (UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
	{
		for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
		{
			TStrongObjectPtr<UObject> Object = WeakObject.Pin();			
			UMediaOutput* MediaOutput = Object.IsValid() ? Cast<UMediaOutput>(Object.Get()) : nullptr;
			if (!MediaOutput)
			{
				continue;
			}

			if (MediaOutput->IsA<UDummyMediaOutput>())
			{
				continue;
			}

			using namespace UE::MediaFrameworkWorldSettings::Helpers;
			
			ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(MediaCaptureSettings, MediaOutput,
				[&MediaCaptureSettings, &CaptureSettingStructs, &bHasCurrentViewportCapture](FMediaFrameworkCaptureCurrentViewportOutputInfo& OutputInfo)
				{
					TSharedPtr<FStructOnScope> Struct = MakeShared<FStructOnScope>(
						FMediaFrameworkCaptureCurrentViewportOutputInfo::StaticStruct(),
						reinterpret_cast<uint8*>(&OutputInfo));
					Struct->SetPackage(MediaCaptureSettings->GetPackage());
						
					CaptureSettingStructs.Add(Struct);
					bHasCurrentViewportCapture = true;
				});
			
			ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(MediaCaptureSettings, MediaOutput,
				[&MediaCaptureSettings, &CaptureSettingStructs, &bHasViewportCaptures](FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo)
				{
					TSharedPtr<FStructOnScope> Struct = MakeShared<FStructOnScope>(FMediaFrameworkCaptureCameraViewportCameraOutputInfo::StaticStruct(), reinterpret_cast<uint8*>(&OutputInfo));
					Struct->SetPackage(MediaCaptureSettings->GetPackage());
					
					CaptureSettingStructs.Add(Struct);
					bHasViewportCaptures = true;
				});

			ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(MediaCaptureSettings, MediaOutput,
				[&MediaCaptureSettings, &CaptureSettingStructs, &bHasRenderTargetCaptures](FMediaFrameworkCaptureRenderTargetCameraOutputInfo& OutputInfo)
				{
						TSharedPtr<FStructOnScope> Struct = MakeShared<FStructOnScope>(FMediaFrameworkCaptureRenderTargetCameraOutputInfo::StaticStruct(), reinterpret_cast<uint8*>(&OutputInfo));
						Struct->SetPackage(MediaCaptureSettings->GetPackage());
						
						CaptureSettingStructs.Add(Struct);
						bHasRenderTargetCaptures = true;
				});

			ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(MediaCaptureSettings, MediaOutput,
				[&MediaCaptureSettings, &CaptureSettingStructs, &bHasMediaTextureCaptures](FMediaFrameworkCaptureMediaTextureOutputInfo& OutputInfo)
				{
					TSharedPtr<FStructOnScope> Struct = MakeShared<FStructOnScope>(FMediaFrameworkCaptureMediaTextureOutputInfo::StaticStruct(), reinterpret_cast<uint8*>(&OutputInfo));
					Struct->SetPackage(MediaCaptureSettings->GetPackage());
					
					CaptureSettingStructs.Add(Struct);
					bHasMediaTextureCaptures = true;
				});
		}
	}

	// Only show the capture settings if all objects have capture settings of the same type (current viewport, media viewport, or render target)
	const int32 NumTrueValues = (int32)bHasCurrentViewportCapture + (int32)bHasViewportCaptures + (int32)bHasRenderTargetCaptures + (int32)bHasMediaTextureCaptures;
	if (CaptureSettingStructs.Num() && NumTrueValues <= 1)
	{
		FAddPropertyParams AddPropertyParams;
		AddPropertyParams.HideRootObjectNode(true);
			
		MediaCaptureStruct = MakeShared<FStructOnScopeStructureDataProvider>(CaptureSettingStructs);
		MediaCaptureCategory.AddExternalStructureProperty(MediaCaptureStruct, NAME_None, EPropertyLocation::Default, AddPropertyParams);
	}
}

FText FMediaProfileMediaOutputDetailCustomization::GetMediaItemTypeText() const
{
	return LOCTEXT("MediaOutputTypeText", "Media Output");
}

void FMediaProfileMediaOutputDetailCustomization::RegisterMediaTypeSection(FPropertyEditorModule& PropertyModule, const FName& MediaTypeCategory)
{
	static bool bRegistered = false;
	if (!bRegistered)
	{
		const TSharedPtr<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			UMediaOutput::StaticClass()->GetFName(),
			TEXT("MediaOutputSection"),
			GetMediaItemTypeText());

		Section->AddCategory(MediaTypeCategory);
		bRegistered = true;
	}
}

FText FMediaProfileMediaOutputDetailCustomization::GetMediaObjectLabel(UMediaOutput* InMediaObject) const
{
	if (!MediaProfile.IsValid())
	{
		return FText::GetEmpty();
	}

	int32 Index = MediaProfile->FindMediaOutputIndex(InMediaObject);
	if (InMediaObject->IsA<UDummyMediaOutput>())
	{
		Index = Cast<UDummyMediaOutput>(InMediaObject)->MediaProfileIndex;
	}
		
	return FText::FromString(MediaProfile->GetLabelForMediaOutput(Index));
}

void FMediaProfileMediaOutputDetailCustomization::SetMediaObjectLabel(UMediaOutput* InMediaObject, const FText& InLabel)
{
	if (!MediaProfile.IsValid())
	{
		return;
	}
	
	int32 Index = MediaProfile->FindMediaOutputIndex(InMediaObject);
	if (InMediaObject->IsA<UDummyMediaOutput>())
	{
		Index = Cast<UDummyMediaOutput>(InMediaObject)->MediaProfileIndex;
	}
		
	MediaProfile->SetLabelForMediaOutput(Index, InLabel.ToString());
}

void FMediaProfileMediaOutputDetailCustomization::SetMediaObject(UMediaOutput* InOriginalMediaObject, UMediaOutput* InNewMediaObject)
{
	TSharedPtr<FMediaProfileEditor> PinnedMediaProfileEditor = MediaProfileEditor.Pin();
	if (!PinnedMediaProfileEditor.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return;
	}
	
	int32 Index = PinnedMediaProfile->FindMediaOutputIndex(InOriginalMediaObject);
	if (InOriginalMediaObject->IsA<UDummyMediaOutput>())
	{
		Index = Cast<UDummyMediaOutput>(InOriginalMediaObject)->MediaProfileIndex;
	}

	PinnedMediaProfile->GetPlaybackManager()->CloseOutputFromIndex(Index);
	
	UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
	if (!CaptureSettings)
	{
		return;
	}

	bool bHasExistingCaptureInfo = false;
	auto UpdateMediaOutputRef = [&InNewMediaObject, &bHasExistingCaptureInfo]<typename TOutputInfo>(TOutputInfo& OutputInfo)
	{
		OutputInfo.MediaOutput = InNewMediaObject;
		bHasExistingCaptureInfo = true;
	};

	using namespace UE::MediaFrameworkWorldSettings::Helpers;
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(CaptureSettings, InOriginalMediaObject, UpdateMediaOutputRef);
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(CaptureSettings, InOriginalMediaObject, UpdateMediaOutputRef);
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(CaptureSettings, InOriginalMediaObject, UpdateMediaOutputRef);
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(CaptureSettings, InOriginalMediaObject, UpdateMediaOutputRef);

	if (!bHasExistingCaptureInfo && MediaProfilePlaybackManager::IsActiveViewportCaptureAllowed())
	{
		CaptureSettings->CurrentViewportMediaOutput.MediaOutput = InNewMediaObject;
	}
	
	CaptureSettings->SaveConfig();
	PinnedMediaProfile->SetMediaOutput(Index, InNewMediaObject);
}

void FMediaProfileMediaOutputDetailCustomization::RegisterCaptureSection(FPropertyEditorModule& PropertyModule, const FName& CaptureCategory)
{
	static bool bRegistered = false;
	if (!bRegistered)
	{
		const TSharedPtr<FPropertySection> Section = PropertyModule.FindOrCreateSection(
			UMediaOutput::StaticClass()->GetFName(),
			TEXT("CaptureSection"),
			LOCTEXT("CaptureSection", "Capture"));

		Section->AddCategory(CaptureCategory);
		bRegistered = true;
	}
}

void FMediaProfileMediaOutputDetailCustomization::OnCaptureMethodChanged()
{
	TSharedPtr<FMediaProfileEditor> PinnedMediaProfileEditor = MediaProfileEditor.Pin();
	if (!PinnedMediaProfileEditor.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return;
	}
	
	TArray<TWeakObjectPtr<UObject>> Objects;
	CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

	for (const TWeakObjectPtr<UObject>& Object : Objects)
	{
		TStrongObjectPtr<UObject> PinnedObject = Object.Pin();
		if (!PinnedObject.IsValid())
		{
			continue;
		}

		if (!PinnedObject->IsA<UMediaOutput>())
		{
			continue;
		}

		if (PinnedObject->IsA<UDummyMediaOutput>())
		{
			continue;
		}

		UMediaOutput* MediaOutput = Cast<UMediaOutput>(PinnedObject.Get());
		PinnedMediaProfile->GetPlaybackManager()->CloseOutput(MediaOutput);
		PinnedMediaProfileEditor->GetOnCaptureMethodChanged().Broadcast(MediaOutput);
	}
	
	CachedDetailBuilder.Pin()->ForceRefreshDetails();
}

void FMediaProfileMediaOutputDetailCustomization::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (UMediaProfileEditorCaptureSettings* CaptureSettings = Cast<UMediaProfileEditorCaptureSettings>(InObject))
	{
		if (CaptureSettings != FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			return;
		}
		
		const FName PropertyName = InPropertyChangedEvent.GetPropertyName();
		
		// Refresh the details panel if the MediaOutput property on any captures or the viewport or render target captures lists have changed,
		// as the capture properties may now be invalid
		if (PropertyName == TEXT("MediaOutput") ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMediaFrameworkWorldSettingsAssetUserData, ViewportCaptures) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMediaFrameworkWorldSettingsAssetUserData, RenderTargetCaptures) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMediaFrameworkWorldSettingsAssetUserData, MediaTextureCaptures))
		{
			CachedDetailBuilder.Pin()->ForceRefreshDetails();
		}
		
		static TSet<FName> RefreshableProperties = 
		{
			GET_MEMBER_NAME_CHECKED(FMediaCaptureTransform, Rotation),
			GET_MEMBER_NAME_CHECKED(FMediaCaptureTransform, bFlipHorizontal),
			GET_MEMBER_NAME_CHECKED(FMediaCaptureTransform, bFlipVertical),
			GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, Crop),
			GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, ResizeMethod),
			GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, bConvertToDesiredPixelFormat),
			GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, bForceAlphaToOneOnConversion),
			GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, bApplyLinearToSRGBConversion)
		};
		
		if (RefreshableProperties.Contains(PropertyName))
		{
			if (TSharedPtr<FMediaProfileEditor> PinnedMediaProfileEditor = MediaProfileEditor.Pin())
			{
				TArray<TWeakObjectPtr<UObject>> Objects;
				if (TSharedPtr<IDetailLayoutBuilder> DetailBuilder = CachedDetailBuilder.Pin())
				{
					CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);
				}

				for (const TWeakObjectPtr<UObject>& Object : Objects)
				{
					TStrongObjectPtr<UObject> PinnedObject = Object.Pin();
					if (!PinnedObject.IsValid())
					{
						continue;
					}

					if (!PinnedObject->IsA<UMediaOutput>())
					{
						continue;
					}

					if (PinnedObject->IsA<UDummyMediaOutput>())
					{
						continue;
					}

					UMediaOutput* MediaOutput = Cast<UMediaOutput>(PinnedObject.Get());
					PinnedMediaProfileEditor->RestartActiveMediaCaptures(MediaOutput);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
