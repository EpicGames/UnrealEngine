// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DummyMediaObject.h"
#include "EditorCategoryUtils.h"
#include "IDetailCustomization.h"
#include "MediaProfileEditor.h"
#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "Profile/MediaProfile.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "FMediaProfileMediaItemDetailCustomization"

class UMediaPlayer;
class FStructOnScopeStructureDataProvider;

/**
 * Base class for a details panel customization for a media source or output within a media profile. Displays an additional category that
 * contains a editable label field and a type dropdown that allows the user to change the media item's type
 */
template<typename TMediaType>
class FMediaProfileMediaItemDetailCustomization : public IDetailCustomization
{
private:
	class FMediaTypeClassFilter : public IClassViewerFilter
	{
	public:
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			if (InClass != nullptr)
			{
				return InClass->IsChildOf(TMediaType::StaticClass()) &&
					!InClass->HasAnyClassFlags(CLASS_Abstract) &&
					!(InClass == UDummyMediaSource::StaticClass() || InClass == UDummyMediaOutput::StaticClass());
			}
			return false;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return InUnloadedClassData->IsChildOf(TMediaType::StaticClass()) && !InUnloadedClassData->HasAnyClassFlags(CLASS_Abstract);
		}
	};
	
public:
	FMediaProfileMediaItemDetailCustomization(TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor, UMediaProfile* InMediaProfile, FSimpleDelegate InOnObjectsChanged)
	{
		MediaProfileEditor = InMediaProfileEditor;
		MediaProfile = InMediaProfile;
		OnObjectsChanged = InOnObjectsChanged;
	}

	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override
	{
		CachedDetailBuilder = DetailBuilder;
		IDetailCustomization::CustomizeDetails(DetailBuilder);
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		
		// Register any unregistered categories as sections to give it a section pill. Need to do this every time as this details panel can show
		// any number of different subtypes of UMediaSource or UMediaOutput, which may have its own categories
		AddSectionsForEachCategory(DetailBuilder, PropertyModule);
		
		const FText CategoryLabel = FText::Format(LOCTEXT("MediaItemCategoryName", "{0}"), GetMediaItemTypeText());
		const FName MediaTypeCategoryName = TEXT("MediaTypeCategory");
		IDetailCategoryBuilder& MediaTypeCategory = DetailBuilder.EditCategory(MediaTypeCategoryName, CategoryLabel, ECategoryPriority::Important);
		RegisterMediaTypeSection(PropertyModule, MediaTypeCategoryName);
		
		MediaTypeCategory.AddCustomRow(LOCTEXT("LabelFilterText", "Label"))
			.Visibility(TAttribute<EVisibility>::CreateSP(this, &FMediaProfileMediaItemDetailCustomization::GetValidObjectVisibility))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LabelName", "Label"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SEditableTextBox)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FMediaProfileMediaItemDetailCustomization::GetLabelText)
				.IsEnabled(this, &FMediaProfileMediaItemDetailCustomization::IsLabelTextBoxEnabled)
				.OnTextCommitted(this, &FMediaProfileMediaItemDetailCustomization::OnLabelTextCommitted)
			];

		const FText TypeFilterText = FText::Format(LOCTEXT("MediaItemTypeFilterText", "{0} Type"), GetMediaItemTypeText());
		const FText TypeLabel = FText::Format(LOCTEXT("MediaItemTypeName", "{0}"), GetMediaItemTypeText());
		
		MediaTypeCategory.AddCustomRow(TypeFilterText)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(TypeLabel)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.MinDesiredWidth(125.f)
			.MaxDesiredWidth(250.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(ClassPickerComboButton, SComboButton)
					.OnGetMenuContent(this, &FMediaProfileMediaItemDetailCustomization::GetClassPickerMenuContent)
					.ContentPadding(0.f)
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(SImage)
							.Image(this, &FMediaProfileMediaItemDetailCustomization::GetClassIcon)
						]
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock )
							.Text(this, &FMediaProfileMediaItemDetailCustomization::GetClassName)
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				]
			];
	}

protected:
	/** Registers sections to the root media item type for each category being displayed in the details panel if one does not yet exist */
	void AddSectionsForEachCategory(IDetailLayoutBuilder& DetailBuilder, FPropertyEditorModule& PropertyModule)
	{		
		TArray<FName> Categories;
		DetailBuilder.GetCategoryNames(Categories);

		const FName TypeName = TMediaType::StaticClass()->GetFName();
		for (const FName& CategoryName : Categories)
		{
			const TSharedPtr<FPropertySection> Section = PropertyModule.FindOrCreateSection(
				TypeName,
				CategoryName,
				FEditorCategoryUtils::GetCategoryDisplayString(FText::FromName(CategoryName)));

			if (!Section->HasAddedCategory(CategoryName))
			{
				Section->AddCategory(CategoryName);
			}
		}
	}
	
	virtual FText GetMediaItemTypeText() const = 0;

	/** Registers a section for the custom media type category under the type specific name */
	virtual void RegisterMediaTypeSection(FPropertyEditorModule& PropertyModule, const FName& MediaTypeCategory) = 0;
	
	EVisibility GetValidObjectVisibility() const
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

		bool bContainsValidObjects = Objects.ContainsByPredicate([](const TWeakObjectPtr<UObject>& Object)
		{
			return Object.IsValid() && !(Object->IsA<UDummyMediaSource>() || Object->IsA<UDummyMediaOutput>());
		});

		return bContainsValidObjects ? EVisibility::Visible : EVisibility::Hidden;
	}
	
	FText GetLabelText() const
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

		if (Objects.Num() <= 0)
		{
			return FText::GetEmpty();
		}
		
		if (Objects.Num() > 1)
		{
			return LOCTEXT("MultipleValuesText", "Multiple Values");
		}

		TMediaType* MediaObject = Cast<TMediaType>(Objects[0]);
		return GetMediaObjectLabel(MediaObject);
	}
	
	bool IsLabelTextBoxEnabled() const
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

		return Objects.Num() == 1;
	}
	
	void OnLabelTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

		if (Objects.Num() != 1)
		{
			return;
		}
		
		TMediaType* MediaObject = Cast<TMediaType>(Objects[0]);
		
		FScopedTransaction Transaction(LOCTEXT("SetLabelTransaction", "Set Label"));
		MediaProfile->Modify();

		SetMediaObjectLabel(MediaObject, NewText);
	}

	TSharedRef<SWidget> GetClassPickerMenuContent()
	{
		TSharedPtr<FMediaTypeClassFilter> Filter = MakeShared<FMediaTypeClassFilter>();
		
		FClassViewerInitializationOptions Options;
		Options.bShowBackgroundBorder = false;
		Options.bShowUnloadedBlueprints = true;
		Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
		Options.ClassFilters.Add(Filter.ToSharedRef());
		
		return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &FMediaProfileMediaItemDetailCustomization::OnClassPicked));
	}
	
	void OnClassPicked(UClass* InClass)
	{
		ClassPickerComboButton->SetIsOpen(false);

		TArray<TWeakObjectPtr<UObject>> Objects;
		CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

		if (Objects.Num() <= 0)
		{
			return;
		}
		
		FScopedTransaction Transaction(FText::Format(LOCTEXT("SetMediaItemTypeTransaction", "Set {0}"), GetMediaItemTypeText()));
		MediaProfile->Modify();
		
		for (int32 Index = 0; Index < Objects.Num(); ++Index)
		{
			TMediaType* OriginalMediaObject = Cast<TMediaType>(Objects[Index]);
			TMediaType* NewMediaObject = NewObject<TMediaType>(MediaProfile.Get(), InClass, NAME_None, RF_Public | RF_Transactional);
			
			SetMediaObject(OriginalMediaObject, NewMediaObject);
		}

		OnObjectsChanged.ExecuteIfBound();
	}
	
	const FSlateBrush* GetClassIcon() const
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

		if (Objects.Num() != 1)
		{
			return nullptr;
		}

		if (Objects[0]->IsA<UDummyMediaSource>() || Objects[0]->IsA<UDummyMediaOutput>())
		{
			return nullptr;
		}
		
		return FSlateIconFinder::FindIconForClass(Objects[0]->GetClass()).GetIcon();
	}
	
	FText GetClassName() const
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		CachedDetailBuilder.Pin()->GetObjectsBeingCustomized(Objects);

		if (Objects.Num() <= 0)
		{
			return FText::GetEmpty();
		}
		
		if (Objects.Num() > 1)
		{
			return LOCTEXT("MultipleValuesText", "Multiple Values");
		}

		if (Objects[0]->IsA<UDummyMediaSource>() || Objects[0]->IsA<UDummyMediaOutput>())
		{
			return LOCTEXT("NotSetText", "Not Set");
		}
		
		return Objects[0]->GetClass()->GetDisplayNameText();
	}
	
	virtual FText GetMediaObjectLabel(TMediaType* InMediaObject) const = 0;
	virtual void SetMediaObjectLabel(TMediaType* InMediaObject, const FText& InLabel) = 0;
	virtual void SetMediaObject(TMediaType* InOriginalMediaObject, TMediaType* InNewMediaObject) = 0;
	
protected:
	/** Media profile editor that owns the details panel being customized by this customization */
	TWeakPtr<FMediaProfileEditor> MediaProfileEditor;

	/** Media profile that owns the media sources or outputs whose details panel is being customized */
	TWeakObjectPtr<UMediaProfile> MediaProfile;

	/** Cached detail layout builder pointer, used to manually refresh the details panel */
	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;

	/** Combo button that displays the types that the media item can be changed to */
	TSharedPtr<SComboButton> ClassPickerComboButton;

	/** Delegate that is raised when the media items' type is changed */
	FSimpleDelegate OnObjectsChanged;
};

/**
 * Details panel customization for media sources
 */
class FMediaProfileMediaSourceDetailCustomization : public FMediaProfileMediaItemDetailCustomization<UMediaSource>
{
public:
	FMediaProfileMediaSourceDetailCustomization(TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor, UMediaProfile* InMediaProfile, FSimpleDelegate InOnObjectsChanged)
		: FMediaProfileMediaItemDetailCustomization(InMediaProfileEditor, InMediaProfile, InOnObjectsChanged)
	{ }

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
private:
	virtual FText GetMediaItemTypeText() const override;
	virtual void RegisterMediaTypeSection(FPropertyEditorModule& PropertyModule, const FName& MediaTypeCategory) override;
	virtual FText GetMediaObjectLabel(UMediaSource* InMediaObject) const override;
	virtual void SetMediaObjectLabel(UMediaSource* InMediaObject, const FText& InLabel) override;
	virtual void SetMediaObject(UMediaSource* InOriginalMediaObject, UMediaSource* InNewMediaObject) override;

	/** Gets whether the properties of the media player that is playing the media source should be displayed in the details panel */
	bool ShowMediaPlayerOptions(UMediaSource* InSource) const;

	/** Adds appropraite media player properties to the details panel of the media source being displayed */
	void AddMediaPlayerProperties(UMediaSource* InMediaSource, IDetailLayoutBuilder& DetailBuilder);

private:
	/** Weak pointer to the media player of the media source being customized */
	TWeakObjectPtr<UMediaPlayer> SourceMediaPlayer;
};

/**
 * Details customization for media outputs. Adds a media capture category that displays the outputs' media capture settings
 */
class FMediaProfileMediaOutputDetailCustomization : public FMediaProfileMediaItemDetailCustomization<UMediaOutput>
{
public:
	FMediaProfileMediaOutputDetailCustomization(TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor, UMediaProfile* InMediaProfile, FSimpleDelegate InOnObjectsChanged)
		: FMediaProfileMediaItemDetailCustomization(InMediaProfileEditor, InMediaProfile, InOnObjectsChanged)
	{ }

	virtual ~FMediaProfileMediaOutputDetailCustomization() override;
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
private:
	virtual FText GetMediaItemTypeText() const override;
	virtual void RegisterMediaTypeSection(FPropertyEditorModule& PropertyModule, const FName& MediaTypeCategory) override;
	virtual FText GetMediaObjectLabel(UMediaOutput* InMediaObject) const override;
	virtual void SetMediaObjectLabel(UMediaOutput* InMediaObject, const FText& InLabel) override;
	virtual void SetMediaObject(UMediaOutput* InOriginalMediaObject, UMediaOutput* InNewMediaObject) override;

	/** Registers a section for the Capture category */
	void RegisterCaptureSection(FPropertyEditorModule& PropertyModule, const FName& CaptureCategory);

	void OnCaptureMethodChanged();
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
	
private:
	/**
	 * Stored FStructOnScopes for the media capture structs of all objects being displayed by this customization, which are pulled from
	 * the currently active UMediaFrameworkWorldSettingsAssetUserData's ViewportCaptures and RenderTargetCaptures list
	 */
	TSharedPtr<FStructOnScopeStructureDataProvider> MediaCaptureStruct;
};

#undef LOCTEXT_NAMESPACE
