// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaskedMediaImage.h"

#include "SceneView.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"

void SMaskedMediaImage::Construct(const FArguments& InArgs, UTexture* InMediaImageTexture)
{
	MediaImageTexture = InMediaImageTexture;
	ImageSizeAttr = InArgs._ImageSize;
	ChannelMaskAttr = InArgs._ChannelMask;
	InvertAlphaChannelAttr = InArgs._InvertAlphaChannel;
	DrawCheckerboardAttr = InArgs._DrawCheckerboard;
	RotationAttr = InArgs._Rotation;
	ScaleAttr = InArgs._Scale;

	UMaterial* BaseMaterial = (UMaterial*)StaticLoadObject(UMaterial::StaticClass(), nullptr, TEXT("/MediaFrameworkUtilities/Editor/M_MaskedMediaImage.M_MaskedMediaImage"));
	check(BaseMaterial);
	
	Material = TStrongObjectPtr<UMaterialInstanceDynamic>(UMaterialInstanceDynamic::Create(BaseMaterial, nullptr));
	Material->SetTextureParameterValue(TEXT("MediaImage"), InMediaImageTexture);
	Material->SetScalarParameterValue(TEXT("bIsAlphaPremultiplied"), InArgs._IsAlphaPremultiplied ? 1.0f : 0.0f);
	
	MaterialBrush = MakeShared<FSlateBrush>();
	MaterialBrush->SetResourceObject(Material.Get());
	
	ChildSlot
	[
		SNew(SScaleBox)
		.Stretch(EStretch::Fill)
		[
			SNew(SImage)
			.Image(MaterialBrush.IsValid() ? MaterialBrush.Get() : FAppStyle::GetBrush("WhiteTexture"))
		]
	];
}

void SMaskedMediaImage::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const FVector2f Scale = ScaleAttr.Get(FVector2f::UnitVector);
	const float Rotation = RotationAttr.Get(0.0f);

	if (ImageSizeAttr.IsSet())
	{
		const FVector2D Size = ImageSizeAttr.Get();
		const FVector2D RotatedSize = FVector2D(
			Size.X * FMath::Abs(FMath::Cos(Rotation)) + Size.Y * FMath::Abs(FMath::Sin(Rotation)),
			Size.X * FMath::Abs(FMath::Sin(Rotation)) + Size.Y * FMath::Abs(FMath::Cos(Rotation)));
		
		MaterialBrush->ImageSize.X = static_cast<float>(RotatedSize.X);
		MaterialBrush->ImageSize.Y = static_cast<float>(RotatedSize.Y);
	}

	const int8 ChannelMask = (int8)ChannelMaskAttr.Get(EColorChannelMask::All);
	Material->SetScalarParameterValue(TEXT("ChannelMask"), ChannelMask + 1);
	Material->SetScalarParameterValue(TEXT("bInvertAlphaChannelMask"), InvertAlphaChannelAttr.Get(false) ? 1.0f : 0.0f);
	Material->SetScalarParameterValue(TEXT("bDrawCheckerboard"), DrawCheckerboardAttr.Get(false) ? 1.0f : 0.0f);
	
	const FVector4 RotScale = FVector4(
		Scale.X * FMath::Cos(Rotation), -Scale.X * FMath::Sin(Rotation),
		Scale.Y * FMath::Sin(Rotation), Scale.Y * FMath::Cos(Rotation)
	);
	
	Material->SetVectorParameterValue(TEXT("RotScale"), RotScale);
}
