// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Materials/MaterialInstanceDynamic.h"

enum class EColorChannelMask : int8;
class UMaterialInstanceDynamic;
class UTexture;

/**
 * Displays a media image texture with optional channel masking and alpha blending support
 */
class SMaskedMediaImage : public SCompoundWidget
{
public:
	
	
	SLATE_BEGIN_ARGS(SMaskedMediaImage)
		: _IsAlphaPremultiplied(true)
		, _Rotation(0.0f)
		, _Scale(FVector2D(1.0f, 1.0f))
	{ }
		SLATE_ATTRIBUTE(FVector2D, ImageSize)
		SLATE_ATTRIBUTE(EColorChannelMask, ChannelMask)
		SLATE_ATTRIBUTE(bool, InvertAlphaChannel)
		SLATE_ATTRIBUTE(bool, DrawCheckerboard)
		SLATE_ARGUMENT(bool, IsAlphaPremultiplied)
		SLATE_ATTRIBUTE(float, Rotation)
		SLATE_ATTRIBUTE(FVector2f, Scale)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UTexture* InMediaImageTexture);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
private:
	/** Material used to render the media image texture */
	TStrongObjectPtr<UMaterialInstanceDynamic> Material;

	/** The texture the media image is being written to */
	TWeakObjectPtr<UTexture> MediaImageTexture;
	
	/** Slate brush that wraps the material */
	TSharedPtr<FSlateBrush> MaterialBrush;
	
	/** Attribute used to size the media image appropriately */
	TAttribute<FVector2D> ImageSizeAttr;

	/** Attribute for the color channel to mask to */
	TAttribute<EColorChannelMask> ChannelMaskAttr;

	/** Attribute for the invert alpha flag */
	TAttribute<bool> InvertAlphaChannelAttr;

	/** Attribute for drawing the alpha-blended checkerboard background */
	TAttribute<bool> DrawCheckerboardAttr;
	
	/** The 2D rotation of the image */
	TAttribute<float> RotationAttr;
	
	/** The 2D scale of the image */
	TAttribute<FVector2f> ScaleAttr;
};
