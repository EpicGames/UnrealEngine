// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaProfileDetailsInfoPanel.h"

#include "CaptureCardMediaSource.h"
#include "DetailLayoutBuilder.h"
#include "DummyMediaObject.h"
#include "MediaFrameworkWorldSettingsAssetUserData.h"
#include "MediaOutput.h"
#include "MediaPlayer.h"
#include "MediaProfileEditor.h"
#include "MediaProfileEditorUserSettings.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MediaAssets/ProxyMediaOutput.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaObjectInfoPanel"

namespace UE::MediaFrameworkUtilities::Private
{
	static TAutoConsoleVariable<float> CVarStreamInfoRefreshPeriod(
		TEXT("MediaProfileEditor.StreamInfoRefreshPeriod"),
		0.5f,
		TEXT("Period (in seconds) at which the Media Profile editor's Stream Info panel re-reads ")
		TEXT("runtime resolution, frame rate, and format from the active preview player. ")
		TEXT("Takes effect the next time a media source is selected. Values below 0.05 are clamped."),
		ECVF_Default);
}

TSharedRef<SWidget> SMediaObjectInfoPanel::FTwoColumnInfo::CreateWidget()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 12.0f, 2.0f)
		[
			SAssignNew(LabelColumn, SVerticalBox)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SAssignNew(ValueColumn, SVerticalBox)
		];
}

void SMediaObjectInfoPanel::FTwoColumnInfo::AddEntry(TAttribute<FText> Label, TAttribute<FText> Value, TAttribute<FSlateColor> ValueColor)
{
	LabelColumn->AddSlot()
	    .Padding(0.0, 2.0f)
		[
			SNew(STextBlock)
			.Text(Label)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	ValueColumn->AddSlot()
	    .Padding(0.0, 2.0f)
		[
			SNew(STextBlock)
			.Text(Value)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(ValueColor)
		];
}

void SMediaObjectInfoPanel::FTwoColumnInfo::AddEntry(TAttribute<FText> Value, const FSlateBrush* Icon)
{
	LabelColumn->AddSlot()
		.Padding(0.0, 2.0f)
		[
			SNullWidget::NullWidget
		];

	ValueColumn->AddSlot()
		.Padding(0.0, 2.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(16.0f)
				.HeightOverride(16.0f)
				[
					SNew(SImage).Image(Icon)
				]
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(6.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(Value)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

void SMediaObjectInfoPanel::FTwoColumnInfo::AddEntry(TAttribute<FText> Value)
{
	LabelColumn->AddSlot()
		.Padding(0.0, 2.0f)
		[
			SNullWidget::NullWidget
		];

	ValueColumn->AddSlot()
		.Padding(0.0, 2.0f)
		[
			SNew(STextBlock)
			.Text(Value)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

void SMediaObjectInfoPanel::FTwoColumnInfo::ClearEntries()
{
	if (LabelColumn.IsValid())
	{
		LabelColumn->ClearChildren();
	}

	if (ValueColumn.IsValid())
	{
		ValueColumn->ClearChildren();
	}
}

void SMediaObjectInfoPanel::Construct(const FArguments& InArgs, TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor, UMediaProfile* InMediaProfile)
{
	MediaProfileEditor = InMediaProfileEditor;
	MediaProfile = InMediaProfile;
		
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 16.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(0.5f)
			[
				CreateInfoGroupWidget(LOCTEXT("BasicInfoHeader", "Info"), BasicInfo)
			]

			+SHorizontalBox::Slot()
			.FillWidth(0.5f)
			[
				CreateInfoGroupWidget(TAttribute<FText>::CreateLambda([this]()
				{
					return ObjectAsMediaSource() ? LOCTEXT("MediaSourceInfoHeader", "Media Source") : LOCTEXT("MediaOutputInfoHeader", "Media Output");
				}), TypeInfo)
			]
		]
			
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(MediaObjectInfoBox, SHorizontalBox)
		]
	];
}

SMediaObjectInfoPanel::~SMediaObjectInfoPanel()
{
	UnsubscribeFromDetectedConfigurationChanges();
}

void SMediaObjectInfoPanel::SetMediaObject(UObject* InMediaObject)
{
	UnsubscribeFromDetectedConfigurationChanges();

	MediaObject = InMediaObject;
	ClearInfo();
	FillInfo();

	SubscribeToDetectedConfigurationChanges(InMediaObject);

	if (!MediaObject.IsValid())
	{
		SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		SetVisibility(EVisibility::Visible);
	}
}

void SMediaObjectInfoPanel::SubscribeToDetectedConfigurationChanges(UObject* InMediaObject)
{
	// Resolve through proxy sources so we listen on the leaf, matching FillInfo's behavior.
	UObject* LeafObject = InMediaObject;
	if (UProxyMediaSource* ProxySource = Cast<UProxyMediaSource>(LeafObject))
	{
		LeafObject = ProxySource->GetLeafMediaSource();
	}

	if (UCaptureCardMediaSource* CaptureSource = Cast<UCaptureCardMediaSource>(LeafObject))
	{
		SubscribedCaptureSource = CaptureSource;
		DetectedConfigurationChangedHandle = CaptureSource->OnLastDetectedConfigurationChanged().AddSP(
			this, &SMediaObjectInfoPanel::OnDetectedConfigurationChanged);
	}
}

void SMediaObjectInfoPanel::UnsubscribeFromDetectedConfigurationChanges()
{
	if (DetectedConfigurationChangedHandle.IsValid())
	{
		if (UCaptureCardMediaSource* CaptureSource = SubscribedCaptureSource.Get())
		{
			CaptureSource->OnLastDetectedConfigurationChanged().Remove(DetectedConfigurationChangedHandle);
		}
		DetectedConfigurationChangedHandle.Reset();
	}
	SubscribedCaptureSource.Reset();
}

void SMediaObjectInfoPanel::OnDetectedConfigurationChanged()
{
	ClearInfo();
	FillInfo();
}

UMediaSource* SMediaObjectInfoPanel::ObjectAsMediaSource() const
{
	return Cast<UMediaSource>(MediaObject);
}

UMediaOutput* SMediaObjectInfoPanel::ObjectAsMediaOutput() const
{
	return Cast<UMediaOutput>(MediaObject);
}

void SMediaObjectInfoPanel::ClearInfo()
{
	BasicInfo.ClearEntries();
	TypeInfo.ClearEntries();
	CaptureInfo.ClearEntries();
	StreamInfo.ClearEntries();

	if (StreamInfoRefreshTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(StreamInfoRefreshTimerHandle.ToSharedRef());
		StreamInfoRefreshTimerHandle.Reset();
	}

	MediaObjectInfoBox->ClearChildren();
}

void SMediaObjectInfoPanel::FillInfo()
{
	if (!MediaObject.IsValid() || !MediaProfile.IsValid())
	{
		return;
	}

	UObject* ActualMediaObject = MediaObject.Get();
	if (UProxyMediaSource* ProxySource = Cast<UProxyMediaSource>(ActualMediaObject))
	{
		ActualMediaObject = ProxySource->GetLeafMediaSource();
	}
	else if (UProxyMediaOutput* ProxyOutput = Cast<UProxyMediaOutput>(ActualMediaObject))
	{
		ActualMediaObject = ProxyOutput->GetLeafMediaOutput();
	}
		
	FText Label;
	if (UMediaSource* MediaSource = ObjectAsMediaSource())
	{
		int32 Index = MediaProfile->FindMediaSourceIndex(MediaSource);
		if (MediaSource->IsA<UDummyMediaSource>())
		{
			Index = Cast<UDummyMediaSource>(MediaSource)->MediaProfileIndex;
		}
		
		Label = FText::FromString(MediaProfile->GetLabelForMediaSource(Index));
	}
	else if (UMediaOutput* MediaOutput = ObjectAsMediaOutput())
	{
		int32 Index = MediaProfile->FindMediaOutputIndex(MediaOutput);
		if (MediaOutput->IsA<UDummyMediaOutput>())
		{
			Index = Cast<UDummyMediaOutput>(MediaOutput)->MediaProfileIndex;
		}
		
		Label = FText::FromString(MediaProfile->GetLabelForMediaOutput(Index));
	}
		
	BasicInfo.AddEntry(LOCTEXT("MediaObjectNameLabel", "Label:"), Label);

	const bool bIsProxy = MediaObject->IsA<UProxyMediaSource>() || MediaObject->IsA<UProxyMediaOutput>();
	BasicInfo.AddEntry(LOCTEXT("MediaObjectIsProxyLabel", "Is Proxy:"), bIsProxy ? LOCTEXT("Yes", "Yes") : LOCTEXT("No", "No"));

	if (ActualMediaObject)
	{
		TypeInfo.AddEntry(ActualMediaObject->GetClass()->GetDisplayNameText(), FSlateIconFinder::FindIconForClass(ActualMediaObject->GetClass()).GetIcon());
	}
	else
	{
		const FText NoObjectSet = ObjectAsMediaSource() ? LOCTEXT("NoMediaSourceSet", "No media source set") : LOCTEXT("NoMediaOutputSet", "No media output set");
		TypeInfo.AddEntry(NoObjectSet);
	}
		
	TArray<TTuple<FText, FText, FText>> InfoElements;

	if (UMediaSource* MediaSource = Cast<UMediaSource>(ActualMediaObject))
	{
		MediaSource->GetDetailsPanelInfoElements(InfoElements);
	}
	else if (UMediaOutput* MediaOutput = Cast<UMediaOutput>(ActualMediaObject))
	{
		MediaOutput->GetDetailsPanelInfoElements(InfoElements);
	}
		
	TMap<FString, FTwoColumnInfo> InfoWidgets;
	for (TTuple<FText, FText, FText>& InfoElement : InfoElements)
	{
		const FText InfoGroup = InfoElement.Get<0>();
		const FText InfoLabel = InfoElement.Get<1>();
		const FText InfoValue = InfoElement.Get<2>();
			
		if (!InfoWidgets.Contains(InfoGroup.ToString()))
		{
			InfoWidgets.Add(InfoGroup.ToString(), FTwoColumnInfo());
			MediaObjectInfoBox->AddSlot()
			[
				CreateInfoGroupWidget(InfoGroup, InfoWidgets[InfoGroup.ToString()])
			];
		}

		InfoWidgets[InfoGroup.ToString()].AddEntry(FText::Format(LOCTEXT("MediaObjectInfoElementLabel", "{0}:"), InfoLabel), InfoValue);
	}
	
	if (Cast<UMediaOutput>(ActualMediaObject))
	{
		MediaObjectInfoBox->AddSlot()
		[
			CreateInfoGroupWidget(LOCTEXT("CaptureSettingsGroupLabel", "Capture Settings"), CaptureInfo)
		];

		CaptureInfo.AddEntry(LOCTEXT("CaptureMethodLabel", "Capture Method:"), TAttribute<FText>::CreateSP(this, &SMediaObjectInfoPanel::GetCaptureMethodText));
		CaptureInfo.AddEntry(TAttribute<FText>::CreateSP(this, &SMediaObjectInfoPanel::GetCaptureObjectLabelText), TAttribute<FText>::CreateSP(this, &SMediaObjectInfoPanel::GetCaptureObjectValueText));
		CaptureInfo.AddEntry(LOCTEXT("CaptureStatsLabel", "Status:"),
			TAttribute<FText>::CreateSP(this, &SMediaObjectInfoPanel::GetCaptureStatusText),
			TAttribute<FSlateColor>::CreateSP(this, &SMediaObjectInfoPanel::GetCaptureStatusColor));
		CaptureInfo.AddEntry(LOCTEXT("CaptureColorConversionLabel", "Color Conversion:"), TAttribute<FText>::CreateSP(this, &SMediaObjectInfoPanel::GetCaptureColorConversionText));
	}
	else if (Cast<UMediaSource>(ActualMediaObject))
	{
		MediaObjectInfoBox->AddSlot()
		[
			CreateInfoGroupWidget(LOCTEXT("StreamInfoGroupLabel", "Stream Info"), StreamInfo)
		];

		StreamInfo.AddEntry(LOCTEXT("StreamResolutionLabel", "Resolution:"), TAttribute<FText>::CreateSP(this, &SMediaObjectInfoPanel::GetStreamResolutionText));
		StreamInfo.AddEntry(LOCTEXT("StreamFrameRateLabel", "Frame Rate:"), TAttribute<FText>::CreateSP(this, &SMediaObjectInfoPanel::GetStreamFrameRateText));
		StreamInfo.AddEntry(LOCTEXT("StreamFormatLabel", "Format:"), TAttribute<FText>::CreateSP(this, &SMediaObjectInfoPanel::GetStreamFormatText));

		const float RefreshPeriod = FMath::Max(0.05f,
			UE::MediaFrameworkUtilities::Private::CVarStreamInfoRefreshPeriod.GetValueOnGameThread());
		StreamInfoRefreshTimerHandle = RegisterActiveTimer(RefreshPeriod,
			FWidgetActiveTimerDelegate::CreateSP(this, &SMediaObjectInfoPanel::RefreshStreamInfo));
	}
}

TSharedRef<SWidget> SMediaObjectInfoPanel::CreateInfoGroupWidget(const TAttribute<FText>& InHeader, FTwoColumnInfo& InInfo)
{
	return SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(InHeader)
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			InInfo.CreateWidget()
		];			
}

FText SMediaObjectInfoPanel::GetCaptureMethodText() const
{
	UMediaOutput* MediaOutput = ObjectAsMediaOutput();
	UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
	if (!MediaOutput || !CaptureSettings)
	{
		return FText::GetEmpty();
	}

	using namespace UE::MediaFrameworkWorldSettings::Helpers;

	const int32 bHasCurrentViewportCapture = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();
	const int32 bHasViewportCaptures = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();
	const int32 bHasRenderTargetCaptures = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();
	const int32 bHasMediaTextureCaptures = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();

	const int32 NumTrueValues = (int32)bHasCurrentViewportCapture + (int32)bHasViewportCaptures + (int32)bHasRenderTargetCaptures + (int32)bHasMediaTextureCaptures;
	if (NumTrueValues >= 2)
	{
		return LOCTEXT("MultipleCapturesValue", "Multiple Captures");
	}

	if (bHasCurrentViewportCapture)
	{
		return LOCTEXT("CurrentViewportCaptureValue", "Current Viewport");	
	}

	if (bHasViewportCaptures)
	{
		return LOCTEXT("ViewportCaptureValue", "Media Viewport");
	}

	if (bHasRenderTargetCaptures)
	{
		return LOCTEXT("RenderTargetCaptureValue", "Render Target");
	}

	if (bHasMediaTextureCaptures)
	{
		return LOCTEXT("MediaTextureCaptureValue", "Media Texture");
	}

	return LOCTEXT("NoCapturesValues", "No Captures Configured");
}

FText SMediaObjectInfoPanel::GetCaptureObjectLabelText() const
{
	UMediaOutput* MediaOutput = ObjectAsMediaOutput();
	UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
	if (!MediaOutput || !CaptureSettings)
	{
		return FText::GetEmpty();
	}

	using namespace UE::MediaFrameworkWorldSettings::Helpers;

	const int32 bHasCurrentViewportCapture = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();
	const int32 bHasViewportCaptures = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();
	const int32 bHasRenderTargetCaptures = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();
	const int32 bHasMediaTextureCaptures = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();

	const int32 NumTrueValues = (int32)bHasCurrentViewportCapture + (int32)bHasViewportCaptures + (int32)bHasRenderTargetCaptures + (int32)bHasMediaTextureCaptures;
	if (NumTrueValues >= 2)
	{
		return LOCTEXT("MultipleCaptureObjectsLabel", "Multiple Objects:");
	}

	if (bHasCurrentViewportCapture)
	{
		return FText::GetEmpty();
	}

	if (bHasViewportCaptures)
	{
		return LOCTEXT("CameraObjectLabel", "Cameras:");
	}

	if (bHasRenderTargetCaptures)
	{
		return LOCTEXT("RenderTargetObjectLabel", "Render Target:");
	}

	if (bHasMediaTextureCaptures)
	{
		return LOCTEXT("MediaTextureObjectLabel", "Media Texture:");
	}

	return LOCTEXT("NoObjectsLabel", "No Objects:");
}

FText SMediaObjectInfoPanel::GetCaptureObjectValueText() const
{
	UMediaOutput* MediaOutput = ObjectAsMediaOutput();
	UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
	if (!MediaOutput || !CaptureSettings)
	{
		return FText::GetEmpty();
	}

	using namespace UE::MediaFrameworkWorldSettings::Helpers;

	const bool bHasCurrentViewportCapture = !FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(CaptureSettings, MediaOutput).IsEmpty();

	bool bHasViewportCaptures = false;
	FText CamerasText;
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(CaptureSettings, MediaOutput, 
	[&bHasViewportCaptures, &CamerasText](const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& Info)
	{
		bHasViewportCaptures = true;
		
		for (const TSoftObjectPtr<AActor>& Camera : Info.Cameras)
		{
			AActor* Actor = Camera.Get();
			if (Actor)
			{
				if (CamerasText.IsEmpty())
				{
					CamerasText = FText::FromString(Actor->GetActorNameOrLabel());
				}
				else
				{
					CamerasText = FText::Format(LOCTEXT("AppendCameraNameFormat", "{0}, {1}"), CamerasText, FText::FromString(Actor->GetActorNameOrLabel()));
				}
			}
		}
	});

	bool bHasRenderTargetCaptures = false;
	FText RenderTargetsText;
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(CaptureSettings, MediaOutput, 
	[&bHasRenderTargetCaptures, &RenderTargetsText](const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& Info)
	{
		bHasRenderTargetCaptures = true;
		
		FText RenderTargetText = Info.RenderTarget ? FText::FromString(Info.RenderTarget->GetName()) : LOCTEXT("NoRenderTarget", "No Render Target");
		
		if (RenderTargetsText.IsEmpty())
		{
			RenderTargetsText = RenderTargetText;
		}
		else
		{
			RenderTargetsText = FText::Format(LOCTEXT("AppendRenderTargetNameFormat", "{0}, {1}"), RenderTargetsText, RenderTargetText);
		}
	});

	bool bHasMediaTextureCaptures = false;
	FText MediaTexturesText;
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(CaptureSettings, MediaOutput, 
	[&bHasMediaTextureCaptures, &MediaTexturesText](const FMediaFrameworkCaptureMediaTextureOutputInfo& Info)
	{
		bHasMediaTextureCaptures = true;
		
		FText MediaTextureText = Info.MediaTexture ? FText::FromString(Info.MediaTexture->GetName()) : LOCTEXT("NoMediaTexture", "No Media Texture");
		
		if (MediaTexturesText.IsEmpty())
		{
			MediaTexturesText = MediaTextureText;
		}
		else
		{
			MediaTexturesText = FText::Format(LOCTEXT("AppendMediaTextureNameFormat", "{0}, {1}"), MediaTexturesText, MediaTextureText);
		}
	});
	
	const int32 NumTrueValues = (int32)bHasCurrentViewportCapture + (int32)bHasViewportCaptures + (int32)bHasRenderTargetCaptures + (int32)bHasMediaTextureCaptures;
	if (NumTrueValues >= 2)
	{
		return FText::GetEmpty();
	}

	if (bHasViewportCaptures)
	{
		return CamerasText;
	}

	if (bHasRenderTargetCaptures)
	{
		return RenderTargetsText;
	}

	if (bHasMediaTextureCaptures)
	{
		return MediaTexturesText;
	}

	return FText::GetEmpty();
}

FText SMediaObjectInfoPanel::GetCaptureStatusText() const
{
	UMediaOutput* MediaOutput = ObjectAsMediaOutput();
	if (!MediaOutput || !MediaProfile.IsValid())
	{
		return LOCTEXT("UnknownCaptureStatusText", "Unknown");
	}

	return MediaProfile.Pin()->GetPlaybackManager()->IsOutputCapturing(MediaOutput) ? LOCTEXT("CapturingStatusText", "Capturing") : LOCTEXT("NotCapturingStatusText", "Not Capturing");
}

FSlateColor SMediaObjectInfoPanel::GetCaptureStatusColor() const
{
	UMediaOutput* MediaOutput = ObjectAsMediaOutput();
	if (!MediaOutput || !MediaProfile.IsValid())
	{
		return FSlateColor::UseForeground();
	}

	return MediaProfile.Pin()->GetPlaybackManager()->IsOutputCapturing(MediaOutput) ? FSlateColor(FLinearColor(0.0, 1.0, 0.0)) : FSlateColor::UseForeground();
}

FText SMediaObjectInfoPanel::GetCaptureColorConversionText() const
{
	UMediaOutput* MediaOutput = ObjectAsMediaOutput();
	UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
	if (!MediaOutput || !CaptureSettings)
	{
		return LOCTEXT("NoneText", "None");
	}

	FText ColorConversionText;
	auto AppendColorConversionText = [&ColorConversionText]<typename TOutputInfo>(const TOutputInfo& Info)
	{
		if (ColorConversionText.IsEmpty())
		{
			ColorConversionText = FText::FromString(Info.CaptureOptions.GetColorConversionSettingsString());
		}
		else
		{
			ColorConversionText = FText::Format(
				LOCTEXT("AppendColorConversionFormat", "{0}, {1}"),
				ColorConversionText,
				FText::FromString(Info.CaptureOptions.GetColorConversionSettingsString()));
		}
	};

	using namespace UE::MediaFrameworkWorldSettings::Helpers;
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(CaptureSettings, MediaOutput, AppendColorConversionText);
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(CaptureSettings, MediaOutput, AppendColorConversionText);
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(CaptureSettings, MediaOutput, AppendColorConversionText);
	ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(CaptureSettings, MediaOutput, AppendColorConversionText);

	if (ColorConversionText.IsEmpty())
	{
		return LOCTEXT("NoneText", "None");
	}

	return ColorConversionText;
}

UMediaTexture* SMediaObjectInfoPanel::GetActivePreviewMediaTexture() const
{
	UMediaSource* MediaSource = ObjectAsMediaSource();
	if (!MediaSource || !MediaProfile.IsValid())
	{
		return nullptr;
	}

	UMediaProfile* PinnedProfile = MediaProfile.Pin().Get();
	if (!PinnedProfile)
	{
		return nullptr;
	}

	UMediaProfilePlaybackManager* PlaybackManager = PinnedProfile->GetPlaybackManager();
	return PlaybackManager ? PlaybackManager->GetSourceMediaTexture(MediaSource) : nullptr;
}

UMediaPlayer* SMediaObjectInfoPanel::GetActivePreviewMediaPlayer() const
{
	UMediaTexture* Texture = GetActivePreviewMediaTexture();
	return Texture ? Texture->GetMediaPlayer() : nullptr;
}

FText SMediaObjectInfoPanel::GetStreamResolutionText() const
{
	UMediaTexture* Texture = GetActivePreviewMediaTexture();
	if (!Texture)
	{
		return LOCTEXT("StreamValueUnavailable", "-");
	}

	const int32 Width = static_cast<int32>(Texture->GetSurfaceWidth());
	const int32 Height = static_cast<int32>(Texture->GetSurfaceHeight());
	if (Width <= 0 || Height <= 0)
	{
		return LOCTEXT("StreamValueUnavailable", "-");
	}

	FNumberFormattingOptions NumberFormat;
	NumberFormat.SetUseGrouping(false);
	return FText::Format(LOCTEXT("StreamResolutionFormat", "{0} x {1}"),
		FText::AsNumber(Width, &NumberFormat),
		FText::AsNumber(Height, &NumberFormat));
}

FText SMediaObjectInfoPanel::GetStreamFrameRateText() const
{
	UMediaPlayer* Player = GetActivePreviewMediaPlayer();
	if (!Player || !Player->IsReady())
	{
		return LOCTEXT("StreamValueUnavailable", "-");
	}

	const float FrameRate = Player->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);
	if (FrameRate <= 0.0f)
	{
		return LOCTEXT("StreamValueUnavailable", "-");
	}

	FNumberFormattingOptions NumberFormat;
	NumberFormat.SetMinimumFractionalDigits(2).SetMaximumFractionalDigits(3);
	return FText::Format(LOCTEXT("StreamFrameRateFormat", "{0} fps"), FText::AsNumber(FrameRate, &NumberFormat));
}

FText SMediaObjectInfoPanel::GetStreamFormatText() const
{
	UMediaPlayer* Player = GetActivePreviewMediaPlayer();
	if (!Player || !Player->IsReady())
	{
		return LOCTEXT("StreamValueUnavailable", "-");
	}

	const FString TrackType = Player->GetVideoTrackType(INDEX_NONE, INDEX_NONE);
	return TrackType.IsEmpty() ? LOCTEXT("StreamValueUnavailable", "-") : FText::FromString(TrackType);
}

EActiveTimerReturnType SMediaObjectInfoPanel::RefreshStreamInfo(double InCurrentTime, float InDeltaTime)
{
	Invalidate(EInvalidateWidgetReason::Paint);
	return EActiveTimerReturnType::Continue;
}

#undef LOCTEXT_NAMESPACE
