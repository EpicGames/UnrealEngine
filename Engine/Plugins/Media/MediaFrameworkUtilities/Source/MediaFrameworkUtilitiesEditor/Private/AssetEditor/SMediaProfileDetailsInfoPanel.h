// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class FActiveTimerHandle;
class FMediaProfileEditor;
class SVerticalBox;
class SHorizontalBox;
class UMediaOutput;
class UMediaPlayer;
class UMediaProfile;
class UMediaSource;
class UMediaTexture;

/**
 * Info panel that is displayed above the details panel, and displays a summary of a media item's configuration
 */
class SMediaObjectInfoPanel : public SCompoundWidget
{
private:
	/** Helper struct to maintain two columns of widgets, one for labels and one for value content */
	struct FTwoColumnInfo
	{
	public:
		TSharedRef<SWidget> CreateWidget();

		void AddEntry(TAttribute<FText> Label, TAttribute<FText> Value, TAttribute<FSlateColor> ValueColor = TAttribute<FSlateColor>());
		void AddEntry(TAttribute<FText> Value, const FSlateBrush* Icon);
		void AddEntry(TAttribute<FText> Value);

		void ClearEntries();

	private:
		TSharedPtr<SVerticalBox> LabelColumn;
		TSharedPtr<SVerticalBox> ValueColumn;
	};
	
public:
	SLATE_BEGIN_ARGS(SMediaObjectInfoPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FMediaProfileEditor>& InMediaProfileEditor, UMediaProfile* InMediaProfile);

	virtual ~SMediaObjectInfoPanel() override;

	/** Sets the media source or output whose information is being displayed in the panel */
	void SetMediaObject(UObject* InMediaObject);

private:
	/** Subscribe to the leaf source's auto-detected-format change delegate so we can refresh live. */
	void SubscribeToDetectedConfigurationChanges(UObject* InMediaObject);
	/** Unsubscribe from whatever source we last subscribed to (if any). */
	void UnsubscribeFromDetectedConfigurationChanges();
	/** Handler for the broadcast that rebuilds the info panel contents. */
	void OnDetectedConfigurationChanged();

	/** Source we are currently subscribed to and the handle of that subscription. */
	TWeakObjectPtr<class UCaptureCardMediaSource> SubscribedCaptureSource;
	FDelegateHandle DetectedConfigurationChangedHandle;

	UMediaSource* ObjectAsMediaSource() const;
	UMediaOutput* ObjectAsMediaOutput() const;
	
	void ClearInfo();
	void FillInfo();

	TSharedRef<SWidget> CreateInfoGroupWidget(const TAttribute<FText>& InHeader, FTwoColumnInfo& InInfo);

	FText GetCaptureMethodText() const;
	FText GetCaptureObjectLabelText() const;
	FText GetCaptureObjectValueText() const;
	FText GetCaptureStatusText() const;
	FSlateColor GetCaptureStatusColor() const;
	FText GetCaptureColorConversionText() const;

	/** Returns the live preview UMediaTexture that the profile's playback manager has opened for the displayed media source, if any. */
	UMediaTexture* GetActivePreviewMediaTexture() const;
	/** Returns the live preview UMediaPlayer driving the active preview UMediaTexture, if any. */
	UMediaPlayer* GetActivePreviewMediaPlayer() const;

	FText GetStreamResolutionText() const;
	FText GetStreamFrameRateText() const;
	FText GetStreamFormatText() const;

	/** Active timer that drives periodic re-paint of StreamInfo while a media source is displayed. */
	EActiveTimerReturnType RefreshStreamInfo(double InCurrentTime, float InDeltaTime);

private:
	/** Media profile editor that owns this widget */
	TWeakPtr<FMediaProfileEditor> MediaProfileEditor;

	/** Media profile that is being edited */
	TWeakObjectPtr<UMediaProfile> MediaProfile;

	/** Media source or output whose info is being displayed */
	TWeakObjectPtr<UObject> MediaObject;

	/** Container for any extra columns of info */
	TSharedPtr<SHorizontalBox> MediaObjectInfoBox;

	/** Basic info column, which displays media object label and proxy status */
	FTwoColumnInfo BasicInfo;

	/** Column that displays details about the type of the media object */
	FTwoColumnInfo TypeInfo;

	/** Column that displays details about the media capture settings if the object is a media output */
	FTwoColumnInfo CaptureInfo;

	/** Column that displays runtime stream details when the displayed media source has an active preview player */
	FTwoColumnInfo StreamInfo;

	/** Handle for the active timer that re-paints StreamInfo; valid only while the Stream Info group is shown. */
	TSharedPtr<FActiveTimerHandle> StreamInfoRefreshTimerHandle;
};
