// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaProfileDetailsPanel.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DummyMediaObject.h"
#include "IDetailsView.h"
#include "MediaSource.h"
#include "MediaOutput.h"
#include "MediaProfileEditorUserSettings.h"
#include "MediaProfileMediaItemDetailCustomization.h"
#include "SMediaProfileDetailsInfoPanel.h"
#include "SPositiveActionButton.h"
#include "CaptureTab/SMediaFrameworkCapture.h"
#include "Modules/ModuleManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "UI/MediaFrameworkUtilitiesEditorStyle.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SSplitter.h"

#define LOCTEXT_NAMESPACE "MediaProfileDetailsPanel"

void SMediaProfileDetailsPanel::Construct(const FArguments& InArgs, const TSharedPtr<FMediaProfileEditor> InOwningEditor, UMediaProfile* InMediaProfile)
{
	OwningEditor = InOwningEditor;
	MediaProfile = InMediaProfile;
	OnRefresh = InArgs._OnRefresh;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsArgs;
	DetailsArgs.bLockable = true;
	DetailsArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea | FDetailsViewArgs::ComponentsAndActorsUseNameArea;
	DetailsArgs.bCustomNameAreaLocation = true;
	DetailsArgs.bCustomFilterAreaLocation = true;
	DetailsArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsArgs.bShowSectionSelector = true;
	
	DetailsView = PropertyModule.CreateDetailView(DetailsArgs);
	DetailsView->RegisterInstancedCustomPropertyLayout(UMediaSource::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([this]
	{
		return MakeShared<FMediaProfileMediaSourceDetailCustomization>(OwningEditor, MediaProfile.Get(), FSimpleDelegate::CreateSP(this, &SMediaProfileDetailsPanel::RefreshDetailsView));
	}));
	DetailsView->RegisterInstancedCustomPropertyLayout(UMediaOutput::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([this]
	{
		return MakeShared<FMediaProfileMediaOutputDetailCustomization>(OwningEditor, MediaProfile.Get(), FSimpleDelegate::CreateSP(this, &SMediaProfileDetailsPanel::RefreshDetailsView));
	}));
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(10.f, 4.f, 0.f, 0.f)
		.AutoHeight()
		[
			DetailsView->GetNameAreaWidget().ToSharedRef()
		]

		+SVerticalBox::Slot()
		.Padding(10.0f, 16.0f)
		.AutoHeight()
		[
			SAssignNew(InfoPanel, SMediaObjectInfoPanel, OwningEditor, MediaProfile.Get())
		]

		+SVerticalBox::Slot()
		.Padding(6.0f, 4.0f)
		.AutoHeight()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.IsEnabled(this, &SMediaProfileDetailsPanel::IsCaptureButtonEnabled)
			.Visibility(this, &SMediaProfileDetailsPanel::GetCaptureButtonVisibility)
			.OnPressed(this, &SMediaProfileDetailsPanel::OnCaptureButtonPressed)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)				
				[
					SNew(SImage)
					.Image(this, &SMediaProfileDetailsPanel::GetCaptureButtonImage)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SMediaProfileDetailsPanel::GetCaptureButtonText)
				]
			]
		]
		
		+SVerticalBox::Slot()
		[
			SNew(SSplitter)
			.MinimumSlotHeight(80.0f)
			.Orientation(Orient_Vertical)
			.Style(FAppStyle::Get(), "SplitterDark")
			.PhysicalSplitterHandleSize(2.0f)
			+SSplitter::Slot()
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					DetailsView->GetFilterAreaWidget().ToSharedRef()
				]
				
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(6.0f)
				[
					SNew(SPositiveActionButton)
					.Icon(FMediaFrameworkUtilitiesEditorStyle::Get().GetBrush("ToolbarIcon.Refresh"))
					.Text(LOCTEXT("RefreshButtonLabel", "Refresh Media Properties"))
					.ToolTipText(LOCTEXT("RefreshButtonTooltip", "Refreshes the streams of the selected media items to match the item's current property values"))
					.Visibility_Lambda([this]
					{
						return !SelectedMediaSources.IsEmpty() || !SelectedMediaOutputs.IsEmpty() ? EVisibility::Visible : EVisibility::Hidden;
					})
					.OnClicked_Lambda([this]
					{
						OnRefresh.ExecuteIfBound(SelectedMediaSources, SelectedMediaOutputs);
						return FReply::Handled();
					})
				]
				
				+SVerticalBox::Slot()
				[
					DetailsView.ToSharedRef()
				]
			]
		]
	];
}

void SMediaProfileDetailsPanel::SetSelectedMediaItems(const TArray<int32>& InSelectedMediaSourceIndices, const TArray<int32>& InSelectedMediaOutputIndices)
{
	SelectedMediaSources.Empty(InSelectedMediaSourceIndices.Num());
	SelectedMediaSources.Append(InSelectedMediaSourceIndices);
	
	SelectedMediaOutputs.Empty(InSelectedMediaOutputIndices.Num());
	SelectedMediaOutputs.Append(InSelectedMediaOutputIndices);
	
	RefreshDetailsView();
}

void SMediaProfileDetailsPanel::RefreshDetailsView()
{
	DummyMediaObjects.Empty();
	
	if (!MediaProfile.IsValid())
	{
		DetailsView->SetObject(nullptr);
		return;
	}
	
	// Can't display details from sources and outputs at the same time, so clear and return
	if (SelectedMediaSources.Num() > 0 && SelectedMediaOutputs.Num() > 0)
	{
		DetailsView->SetObject(nullptr);
		InfoPanel->SetMediaObject(nullptr);
		return;
	}

	TArray<UObject*> Objects;
	if (SelectedMediaSources.Num() > 0)
	{
		Objects.Reserve(SelectedMediaSources.Num());
		for (int Index = 0; Index < SelectedMediaSources.Num(); ++Index)
		{
			UMediaSource* MediaSource = MediaProfile->GetMediaSource(SelectedMediaSources[Index]);
			if (!MediaSource)
			{
				UDummyMediaSource* DummyMediaSource = NewObject<UDummyMediaSource>();
				DummyMediaObjects.Add(TStrongObjectPtr<UObject>(DummyMediaSource));
				DummyMediaSource->MediaProfileIndex = SelectedMediaSources[Index];

				MediaSource = DummyMediaSource;
			}
			
			Objects.Add(MediaSource);
		}
	}
	else if (SelectedMediaOutputs.Num() > 0)
	{
		Objects.Reserve(SelectedMediaOutputs.Num());
		for (int Index = 0; Index < SelectedMediaOutputs.Num(); ++Index)
		{
			UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(SelectedMediaOutputs[Index]);
			if (!MediaOutput)
			{
				UDummyMediaOutput* DummyMediaOutput = NewObject<UDummyMediaOutput>();
				DummyMediaObjects.Add(TStrongObjectPtr<UObject>(DummyMediaOutput));
				DummyMediaOutput->MediaProfileIndex = SelectedMediaOutputs[Index];

				MediaOutput = DummyMediaOutput;
			}
			
			Objects.Add(MediaOutput);
		}
	}

	if (Objects.Num() > 0)
	{
		DetailsView->SetObjects(Objects);
	}
	else
	{
		DetailsView->SetObject(nullptr);
	}
	
	if (Objects.Num() == 1 && !(Objects[0]->IsA<UDummyMediaSource>() || Objects[0]->IsA<UDummyMediaOutput>()))
	{
		InfoPanel->SetMediaObject(Objects[0]);
	}
	else
	{
		InfoPanel->SetMediaObject(nullptr);
	}
}

void SMediaProfileDetailsPanel::OnCaptureButtonPressed()
{
	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return;
	}
	
	bool bIsCapturing = false;
	for (int32 OutputIndex : SelectedMediaOutputs)
	{
		if (UMediaOutput* MediaOutput = PinnedMediaProfile->GetMediaOutput(OutputIndex))
		{
			if (PinnedMediaProfile->GetPlaybackManager()->IsOutputCapturing(MediaOutput))
			{
				bIsCapturing = true;
				break;
			}
		}
	}
		
	if (bIsCapturing)
	{
		for (int32 OutputIndex : SelectedMediaOutputs)
		{
			if (UMediaOutput* MediaOutput = PinnedMediaProfile->GetMediaOutput(OutputIndex))
			{
				PinnedMediaProfile->GetPlaybackManager()->CloseOutput(MediaOutput);
			}
		}
	}
	else
	{
		UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();
		if (!IsValid(MediaCaptureSettings))
		{
			return;
		}
		
		for (int32 OutputIndex : SelectedMediaOutputs)
		{
			if (UMediaOutput* MediaOutput = PinnedMediaProfile->GetMediaOutput(OutputIndex))
			{
				using namespace UE::MediaFrameworkWorldSettings::Helpers;
				
				if (FMediaFrameworkCaptureCurrentViewportOutputInfo* CurrentViewportCapture =
					FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(MediaCaptureSettings, MediaOutput))
				{
					PinnedMediaProfile->GetPlaybackManager()->OpenActiveViewportOutput(MediaOutput, CurrentViewportCapture->CaptureOptions, MediaCaptureSettings->bAutoRestartCaptureOnChange);
					continue;
				}

				if (FMediaFrameworkCaptureCameraViewportCameraOutputInfo* ViewportCapture =
					FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(MediaCaptureSettings, MediaOutput))
				{
					// Make sure there is a managed viewport before starting the capture
					TSharedPtr<FViewportClient> ViewportClient = PinnedMediaProfile->GetPlaybackManager()->GetOrCreateManagedViewport(MediaOutput, ViewportCapture->ViewMode);
					if (ViewportClient.IsValid())
					{
						PinnedMediaProfile->GetPlaybackManager()->OpenManagedViewportOutput(MediaOutput, ViewportCapture->CaptureOptions, MediaCaptureSettings->bAutoRestartCaptureOnChange);
					}
					continue;
				}

				if (FMediaFrameworkCaptureRenderTargetCameraOutputInfo* RenderTargetCapture =
					FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(MediaCaptureSettings, MediaOutput))
				{
					PinnedMediaProfile->GetPlaybackManager()->OpenRenderTargetOutput(MediaOutput, RenderTargetCapture->RenderTarget, RenderTargetCapture->CaptureOptions, MediaCaptureSettings->bAutoRestartCaptureOnChange);
				}

				if (FMediaFrameworkCaptureMediaTextureOutputInfo* MediaTextureCapture =
					FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(MediaCaptureSettings, MediaOutput))
				{
					// For media texture capture, if the media texture is from a media profile source, make sure the source is open before starting capture
					int32 MediaSourceIndex = INDEX_NONE;
					if (PinnedMediaProfile->GetPlaybackManager()->IsValidSourceMediaTexture(MediaTextureCapture->MediaTexture, MediaSourceIndex))
					{
						PinnedMediaProfile->GetPlaybackManager()->OpenSourceFromIndex(MediaSourceIndex, OwningEditor.Pin().Get());
					}
					
					PinnedMediaProfile->GetPlaybackManager()->OpenMediaTextureOutput(MediaOutput, MediaTextureCapture->MediaTexture, MediaTextureCapture->CaptureOptions, MediaTextureCapture->Transform, MediaCaptureSettings->bAutoRestartCaptureOnChange);
				}
			}
		}
	}
}

const FSlateBrush* SMediaProfileDetailsPanel::GetCaptureButtonImage() const
{
	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return nullptr;
	}
	
	bool bIsCapturing = false;
	for (int32 OutputIndex : SelectedMediaOutputs)
	{
		if (UMediaOutput* MediaOutput = PinnedMediaProfile->GetMediaOutput(OutputIndex))
		{
			if (PinnedMediaProfile->GetPlaybackManager()->IsOutputCapturing(MediaOutput))
			{
				bIsCapturing = true;
				break;
			}
		}
	}
		
	return !bIsCapturing ? FMediaFrameworkUtilitiesEditorStyle::Get().GetBrush("MediaCapture.Capture") : FAppStyle::GetBrush("Icons.Toolbar.Stop");
}

FText SMediaProfileDetailsPanel::GetCaptureButtonText() const
{
	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return FText::GetEmpty();
	}
	
	bool bIsCapturing = false;
	for (int32 OutputIndex : SelectedMediaOutputs)
	{
		if (UMediaOutput* MediaOutput = PinnedMediaProfile->GetMediaOutput(OutputIndex))
		{
			if (PinnedMediaProfile->GetPlaybackManager()->IsOutputCapturing(MediaOutput))
			{
				bIsCapturing = true;
				break;
			}
		}
	}
	
	return !bIsCapturing ? LOCTEXT("MediaOutputStartCapture", "Start Capture") : LOCTEXT("MediaOutputStopCapture", "Stop Capture");
}

EVisibility SMediaProfileDetailsPanel::GetCaptureButtonVisibility() const
{
	TSharedPtr<FMediaProfileEditor> PinnedEditor = OwningEditor.Pin();
	if (!PinnedEditor.IsValid())
	{
		return EVisibility::Collapsed;
	}

	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return EVisibility::Collapsed;
	}

	bool bHasMediaOutputs = false;
	for (int32 OutputIndex : SelectedMediaOutputs)
	{
		if (OutputIndex > INDEX_NONE && OutputIndex < PinnedMediaProfile->NumMediaOutputs())
		{
			bHasMediaOutputs = true;
			break;
		}
	}

	return bHasMediaOutputs ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SMediaProfileDetailsPanel::IsCaptureButtonEnabled() const
{
	TSharedPtr<FMediaProfileEditor> PinnedEditor = OwningEditor.Pin();
	if (!PinnedEditor.IsValid())
	{
		return false;
	}

	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return false;
	}
	
	bool bIsCapturing = false;
	bool bIsInvalidCaptureConfig = false;
	for (int32 OutputIndex : SelectedMediaOutputs)
	{
		if (UMediaOutput* MediaOutput = PinnedMediaProfile->GetMediaOutput(OutputIndex))
		{
			if (PinnedMediaProfile->GetPlaybackManager()->IsOutputCapturing(MediaOutput))
			{
				bIsCapturing = true;
			}
					
			if (!PinnedEditor->CanMediaOutputCapture(MediaOutput))
			{
				bIsInvalidCaptureConfig = true;
			}
		}
	}
	
	return bIsCapturing || !bIsInvalidCaptureConfig;
}

#undef LOCTEXT_NAMESPACE