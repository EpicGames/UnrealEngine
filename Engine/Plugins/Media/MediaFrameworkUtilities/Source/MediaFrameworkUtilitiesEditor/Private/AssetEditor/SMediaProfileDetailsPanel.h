// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MediaProfileEditor.h"
#include "MediaSource.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SMediaObjectInfoPanel;
class IDetailsView;
class UMediaProfile;

/** Details panel for the media profile's selected media sources and outputs */
class SMediaProfileDetailsPanel : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnDetailsChanged);
	DECLARE_DELEGATE_TwoParams(FOnRefreshMediaItems, const TArray<int32>& /* InMediaSources */, const TArray<int32>& /* InMediaOutputs */)
	
public:
	SLATE_BEGIN_ARGS(SMediaProfileDetailsPanel) {}
		SLATE_EVENT(FOnDetailsChanged, OnDetailsChanged)
		SLATE_EVENT(FOnRefreshMediaItems, OnRefresh)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedPtr<FMediaProfileEditor> InOwningEditor, UMediaProfile* InMediaProfile);

	/**
	 * Sets the media items in the media profile to display in the details panel
	 * @param InSelectedMediaSourceIndices A list of indices of the media sources in the media profile to display
	 * @param InSelectedMediaOutputIndices A list of indices of the media outputs in the media profile to display
	 */
	void SetSelectedMediaItems(const TArray<int32>& InSelectedMediaSourceIndices, const TArray<int32>& InSelectedMediaOutputIndices);

private:
	void RefreshDetailsView();

	/** Raised when the Start/Stop Capture button is pressed */
	void OnCaptureButtonPressed();

	/** Gets the icon to display in the capture button, depending on if there is a live capture or not */
	const FSlateBrush* GetCaptureButtonImage() const;

	/** Gets the text to display in the capture button, depending on if there is a live capture or not */
	FText GetCaptureButtonText() const;

	/** Gets the visibility of the capture button, depending on the types of the objects being displayed in the details panel */
	EVisibility GetCaptureButtonVisibility() const;

	/** Gets whether the capture button is enabled, depending on whether the selected outputs are properly configured to capture */
	bool IsCaptureButtonEnabled() const;
	
private:
	/** The editor that owns this details panel */
	TWeakPtr<FMediaProfileEditor> OwningEditor;
	
	/** The media profile being edited by this editor */
	TWeakObjectPtr<UMediaProfile> MediaProfile;
	
	/** Details view to display the media source or output properties */
	TSharedPtr<IDetailsView> DetailsView;

	/** Panel to display info on the selected media objects */
	TSharedPtr<SMediaObjectInfoPanel> InfoPanel;

	/** Strong pointers for any dummy objects needed to represent null media entries in the media profile */
	TArray<TStrongObjectPtr<UObject>> DummyMediaObjects;

	/** Indices of the selected media sources within the media profile to display in the details panel */
	TArray<int32> SelectedMediaSources;

	/** Indices of the selected media outputs within the media profile to display in the details panel */
	TArray<int32> SelectedMediaOutputs;

	/** Raised when the refresh button is pressed */
	FOnRefreshMediaItems OnRefresh;
};
