// Copyright Epic Games, Inc. All Rights Reserved.


#include "SMediaProfileMediaItemDisplay.h"

#include "IMediaProfileModule.h"
#include "LevelEditorViewport.h"
#include "MediaFrameworkWorldSettingsAssetUserData.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "MediaTextureTracker.h"
#include "ShowFlagMenuCommands.h"
#include "SMaskedMediaImage.h"
#include "CaptureTab/SMediaFrameworkCapture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "Providers/ViewportTileVisibilityProvider.h"
#include "Slate/SceneViewport.h"
#include "UI/MediaFrameworkUtilitiesEditorStyle.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/SViewport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaProfileMediaItemDisplay"

void SMediaProfileMediaSourceDisplay::Construct(const FArguments& InArgs, const TSharedPtr<SMediaProfileViewport>& InOwningViewport)
{
	SMediaProfileMediaItemDisplayBase::FArguments BaseArgs;
	BaseArgs.MediaProfileEditor(InArgs._MediaProfileEditor);
	BaseArgs.PanelIndex(InArgs._PanelIndex);
	BaseArgs.MediaItemIndex(InArgs._MediaItemIndex);
	SMediaProfileMediaItemDisplayBase::Construct(BaseArgs, InOwningViewport);

	Overlay->AddSlot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Bottom)
	.Padding(16.0f)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 16.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMediaProfileMediaSourceDisplay::GetTimeDurationText)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SMediaProfileMediaSourceDisplay::GetFramerateText)
		]
	];
}

void SMediaProfileMediaSourceDisplay::ConfigureMediaImage()
{
	// Tear down any prior registration before we potentially switch to a different source slot.
	// Idempotent on first call (nothing registered).
	TeardownTileVisibilityProvider();

	// Forget the previous masked-image widget; it will be rebuilt below if we set up content.
	// Without this we'd read stale geometry from a widget no longer in the tree.
	MediaImage.Reset();

	UMediaSource* MediaSource = GetMediaItem();
	if (!MediaSource)
	{
		MediaImageContainer->SetContent(
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MediaSourceNotConfiguredLabel", "Media Source not configured"))
			]);
		return;
	}

	UMediaProfile* MediaProfile = GetMediaProfile();
	if (!MediaProfile)
	{
		MediaImageContainer->SetContent(SNullWidget::NullWidget);
		return;
	}

	UMediaProfilePlaybackManager* PlaybackManager = MediaProfile->GetPlaybackManager();
	UMediaTexture* MediaTexture = PlaybackManager->GetSourceMediaTextureFromIndex(MediaItemIndex);
	
	// Setup the tile visibility provider before opening the media source so that first video frame has correct tile selection. 
	if (MediaTexture)
	{
		SetupTileVisibilityProvider(MediaTexture);
	}

	// Only attempt to open the media source if it is closed. If it needs a full refresh (e.g. from a property change) the root media profile editor will handle it
	if (!PlaybackManager->IsSourceOpenFromIndex(MediaItemIndex))
	{
		// Open the media source, registering the media profile editor as a consumer
		PlaybackManager->OpenSourceFromIndex(MediaItemIndex, MediaProfileEditor.Pin().Get());
	}

	if (!MediaTexture)
	{
		MediaImageContainer->SetContent(
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CouldNotPlayMediaSourceLabel", "Could not play Media Source"))
			]);
		return;
	}

	const FVector2D TextureSize = FVector2D(MediaTexture->GetSurfaceWidth(), MediaTexture->GetSurfaceHeight());
	MediaImageContainer->SetContent(
		SNew(SBox)
		.MinAspectRatio(this, &SMediaProfileMediaSourceDisplay::GetSourceAspectRatio)
		.MaxAspectRatio(this, &SMediaProfileMediaSourceDisplay::GetSourceAspectRatio)
		[
			SAssignNew(MediaImage, SMaskedMediaImage, MediaTexture)
			.IsAlphaPremultiplied(false)
			.ChannelMask(this, &SMediaProfileMediaSourceDisplay::GetChannelMask)
			.InvertAlphaChannel(this, &SMediaProfileMediaSourceDisplay::GetInvertAlphaChannelMask)
			.DrawCheckerboard(this, &SMediaProfileMediaSourceDisplay::GetDrawAlphaBlendedCheckerboard)
			.ImageSize(TextureSize)
			.Rotation(this, &SMediaProfileMediaSourceDisplay::GetRotationValue, false)
			.Scale(this, &SMediaProfileMediaSourceDisplay::GetScale)
		]);
}

UMediaSource* SMediaProfileMediaSourceDisplay::GetMediaItem() const
{
	UMediaProfile* MediaProfile = GetMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}

	return MediaProfile->GetMediaSource(MediaItemIndex);
}

FText SMediaProfileMediaSourceDisplay::GetMediaItemLabel() const
{
	UMediaProfile* MediaProfile = GetMediaProfile();
	if (!MediaProfile)
	{
		return FText::GetEmpty();
	}

	return FText::FromString(MediaProfile->GetLabelForMediaSource(MediaItemIndex));
}

FText SMediaProfileMediaSourceDisplay::GetBaseMediaTypeLabel() const
{
	return LOCTEXT("MediaSourceBaseTypeLabel", "Media Source");
}

bool SMediaProfileMediaSourceDisplay::CanTransformMediaItem() const
{
	return GetMediaItem() != nullptr;
}

UMediaTexture* SMediaProfileMediaSourceDisplay::GetMediaTexture() const
{
	UMediaProfile* MediaProfile = GetMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	return MediaProfile->GetPlaybackManager()->GetSourceMediaTextureFromIndex(MediaItemIndex);
}

FVector2D SMediaProfileMediaSourceDisplay::GetSourceImageSize() const
{
	if (UMediaTexture* MediaTexture = GetMediaTexture())
	{
		return FVector2D(MediaTexture->GetSurfaceWidth(), MediaTexture->GetSurfaceHeight());
	}

	return FVector2D::ZeroVector;
}

FOptionalSize SMediaProfileMediaSourceDisplay::GetSourceAspectRatio() const
{
	const float Rotation = GetRotationValue();
	const FVector2D ImageSize = GetSourceImageSize();
	const FVector2D RotatedImageSize = FVector2D(
		ImageSize.X * FMath::Abs(FMath::Cos(Rotation)) + ImageSize.Y * FMath::Abs(FMath::Sin(Rotation)),
		ImageSize.X * FMath::Abs(FMath::Sin(Rotation)) + ImageSize.Y * FMath::Abs(FMath::Cos(Rotation)));
	
	return (RotatedImageSize.X > 0 && RotatedImageSize.Y > 0) ? FOptionalSize(RotatedImageSize.X / RotatedImageSize.Y) : FOptionalSize();
}

FText SMediaProfileMediaSourceDisplay::GetTimeDurationText() const
{
	UMediaTexture* MediaTexture = GetMediaTexture();
	if (!MediaTexture)
	{
		return FText::GetEmpty();
	}

	UMediaPlayer* MediaPlayer = MediaTexture->GetMediaPlayer();
	if (!MediaPlayer)
	{
		return FText::GetEmpty();
	}

	FTimespan Time = MediaPlayer->GetTime();
	FTimespan Duration = MediaPlayer->GetDuration();

	return FText::Format(LOCTEXT("MediaSourceTimeDurationFormat", "{0} / {1}"),
		FText::FromString(Time.ToString(TEXT("%h:%m:%s.%f"))),
		Duration == Duration.MaxValue() ? LOCTEXT("InfinitySymbol", "\u221E") : FText::FromString(Duration.ToString(TEXT("%h:%m:%s.%f"))));
}

FText SMediaProfileMediaSourceDisplay::GetFramerateText() const
{
	UMediaTexture* MediaTexture = GetMediaTexture();
	if (!MediaTexture)
	{
		return FText::GetEmpty();
	}

	UMediaPlayer* MediaPlayer = MediaTexture->GetMediaPlayer();
	if (!MediaPlayer)
	{
		return FText::GetEmpty();
	}
	
	return FText::Format(LOCTEXT("MediaSourceFramerateFormat", "{0} fps"), MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE));
}

SMediaProfileMediaSourceDisplay::~SMediaProfileMediaSourceDisplay()
{
	TeardownTileVisibilityProvider();
}

void SMediaProfileMediaSourceDisplay::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SMediaProfileMediaItemDisplayBase::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (TileVisibilityProvider.IsValid())
	{
		// Use the masked-image leaf's tick-space geometry: it reflects the aspect-ratio-constrained
		// rendered area and tracks resizes on the same Tick. Fall back to the parent's allotted
		// geometry on the first Tick before the child tree has been arranged.
		FVector2D AbsSize = FVector2D(AllottedGeometry.GetAbsoluteSize());
		if (MediaImage.IsValid())
		{
			const FVector2D ChildSize = FVector2D(MediaImage->GetTickSpaceGeometry().GetAbsoluteSize());
			if (ChildSize.X > 0.0 && ChildSize.Y > 0.0)
			{
				AbsSize = ChildSize;
			}
		}
		TileVisibilityProvider->SetDisplaySizePx(FIntPoint(
			FMath::CeilToInt32(AbsSize.X),
			FMath::CeilToInt32(AbsSize.Y)));
	}
}

void SMediaProfileMediaSourceDisplay::SetupTileVisibilityProvider(UMediaTexture* InMediaTexture)
{
	if (!InMediaTexture)
	{
		return;
	}

	if (!TileVisibilityProvider.IsValid())
	{
		TileVisibilityProvider = MakeShared<UE::MediaPlayerEditor::FViewportTileVisibilityProvider, ESPMode::ThreadSafe>();
	}

	if (!TextureTrackerObject.IsValid())
	{
		TextureTrackerObject = MakeShared<FMediaTextureTrackerObject, ESPMode::ThreadSafe>();
	}
	TextureTrackerObject->TileVisibilityProvider = TileVisibilityProvider;

	RegisteredMediaTexture = InMediaTexture;
	FMediaTextureTracker::Get().RegisterTexture(TextureTrackerObject, InMediaTexture);
}

void SMediaProfileMediaSourceDisplay::TeardownTileVisibilityProvider()
{
	if (TextureTrackerObject.IsValid())
	{
		if (UMediaTexture* Texture = RegisteredMediaTexture.Get())
		{
			FMediaTextureTracker::Get().UnregisterTexture(TextureTrackerObject, Texture);
		}
		TextureTrackerObject->TileVisibilityProvider.Reset();
		TextureTrackerObject.Reset();
	}
	RegisteredMediaTexture.Reset();
	TileVisibilityProvider.Reset();
}

/** Base widget for rendering the output of a media capture */
class SMediaCaptureImageBase : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaCaptureImageBase) {}
		SLATE_ATTRIBUTE(TOptional<EMediaCaptureState>, MediaCaptureState)
		SLATE_ATTRIBUTE(FOptionalSize, AspectRatio)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		MediaCaptureState = InArgs._MediaCaptureState;

		ChildSlot
		[
			SNew(SBox)
			.MinAspectRatio(InArgs._AspectRatio)
			.MaxAspectRatio(InArgs._AspectRatio)
			[
				SNew(SOverlay)

				+SOverlay::Slot()
				[
					GetContentWidget()
				]
				
				+SOverlay::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.Padding(16)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(SImage).Image(this, &SMediaCaptureImageBase::GetLiveCaptureIcon)
						.ColorAndOpacity(this, &SMediaCaptureImageBase::GetLiveCaptureIconColor)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock).Text(this, &SMediaCaptureImageBase::GetLiveCaptureText)
					]
				]
			]
		];
	}
	
	/** Sets the current color channel being displayed in the viewport */
	virtual void SetChannelMask(EColorChannelMask InChannelMask) { }

	/** Sets whether the alpha color channel is inverted when displayed as the color channel mask */
	virtual void SetAlphaInverted(bool bInAlphaInverted) { }

	/** Sets whether to draw the alpha blended checkerboard background when rendering the viewport */
	virtual void SetDrawAlphaBlendedCheckerboard(bool bInDrawAlphaBlendedCheckerboard) { }

protected:
	virtual TSharedRef<SWidget> GetContentWidget() { return SNullWidget::NullWidget; }
	
	const FSlateBrush* GetLiveCaptureIcon() const
	{
		TOptional<EMediaCaptureState> State = MediaCaptureState.Get();
		switch (State.Get(EMediaCaptureState::Stopped))
		{
		case EMediaCaptureState::Capturing:
			return FMediaFrameworkUtilitiesEditorStyle::Get().GetBrush("MediaCapture.Capture");

		case EMediaCaptureState::Error:
			return FAppStyle::Get().GetBrush("Icons.XCircle");

		case EMediaCaptureState::Preparing:
		case EMediaCaptureState::StopRequested:
			return FAppStyle::Get().GetBrush("Icons.AlertCircle");
			
		case EMediaCaptureState::Stopped:
		default:
			return FAppStyle::GetBrush("Icons.Toolbar.Stop");
		}
	}

	FSlateColor GetLiveCaptureIconColor() const
	{
		TOptional<EMediaCaptureState> State = MediaCaptureState.Get();
		switch (State.Get(EMediaCaptureState::Stopped))
		{
		case EMediaCaptureState::Capturing:
		case EMediaCaptureState::Preparing:
		case EMediaCaptureState::Stopped:
		default:
			return FSlateColor::UseForeground();

		case EMediaCaptureState::Error:
			return FStyleColors::Error;

		case EMediaCaptureState::StopRequested:
			return FStyleColors::Warning;
		}
	}
	
	FText GetLiveCaptureText() const
	{
		TOptional<EMediaCaptureState> State = MediaCaptureState.Get();
		switch (State.Get(EMediaCaptureState::Stopped))
		{
		case EMediaCaptureState::Capturing:
			return LOCTEXT("CapturingLabel", "Capturing");

		case EMediaCaptureState::Error:
			return LOCTEXT("ErrorCapturingLabel", "Error occured while capturing");
			
		case EMediaCaptureState::StopRequested:
			return LOCTEXT("StoppingCaptureLabel", "Stopping capture...");

		case EMediaCaptureState::Preparing:
			return LOCTEXT("PreparingCaptureLabel", "Preparing capture...");
			
		case EMediaCaptureState::Stopped:
		default:
			return LOCTEXT("NotCapturingLabel", "Not Capturing");
		}
	}
	
protected:
	/** Attribute for querying the current media capture state of the media output being displayed */
	TAttribute<TOptional<EMediaCaptureState>> MediaCaptureState;
};

/** Widget that outputs the contents of a scene viewport render target */
class SActiveEditorViewportRenderer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActiveEditorViewportRenderer) {}
		SLATE_ATTRIBUTE(TWeakPtr<FSceneViewport>, ActiveEditorViewport)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs)
	{
		ActiveEditorViewport = InArgs._ActiveEditorViewport;

		ChildSlot
		[
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoActiveEditorViewportLabel", "No active editor viewports found"))
			]
		];
	}
	
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		TSharedPtr<FSceneViewport> Viewport = ActiveEditorViewport.IsSet() ? ActiveEditorViewport.Get().Pin() : nullptr;
		if (!Viewport.IsValid() || Viewport->GetViewportRenderTargetTexture() == nullptr)
		{
			// If the viewport is not valid, draw the widget's children, which will display an error message
			return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		}
		
		ESlateDrawEffect DrawEffects = ESlateDrawEffect::None;
		DrawEffects |= ESlateDrawEffect::IgnoreTextureAlpha;
		DrawEffects |= ESlateDrawEffect::NoGamma;
		DrawEffects |= ESlateDrawEffect::NoBlending;
		
		FSlateDrawElement::MakeViewport(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), Viewport, DrawEffects);
		
		return LayerId + 1;
	}

private:
	/**
	 * Attribute to retrieve the active editor viewport used for the media capture. Using weak pointer to avoid TAttribute caching a shared pointer,
	 * as PIE viewports specifically don't like having stray shared pointers floating around
	 */
	TAttribute<TWeakPtr<FSceneViewport>> ActiveEditorViewport;
};

/** Widget that displays a current viewport media capture */
class SMediaCaptureEditorViewportImage : public SMediaCaptureImageBase
{
public:
	SLATE_BEGIN_ARGS(SMediaCaptureEditorViewportImage) {}
		SLATE_ATTRIBUTE(TOptional<EMediaCaptureState>, MediaCaptureState)
		SLATE_ATTRIBUTE(FOptionalSize, AspectRatio)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UMediaProfile* InMediaProfile, const FMediaFrameworkCaptureCurrentViewportOutputInfo& OutputInfo)
	{
		MediaProfile = InMediaProfile;
		MediaOutputIndex = MediaProfile->FindMediaOutputIndex(OutputInfo.MediaOutput);
		AspectRatio = InArgs._AspectRatio;
		
		SMediaCaptureImageBase::FArguments BaseArgs;
		BaseArgs.MediaCaptureState(InArgs._MediaCaptureState);
		BaseArgs.AspectRatio(this, &SMediaCaptureEditorViewportImage::GetAspectRatio);
		
		SMediaCaptureImageBase::Construct(BaseArgs);
	}

	virtual void SetChannelMask(EColorChannelMask InChannelMask) override
	{
		TWeakPtr<FSceneViewport> Viewport = GetActiveEditorViewport();
		if (TSharedPtr<FSceneViewport> PinnedViewport = Viewport.Pin())
		{
			if (FEditorViewportClient* EditorViewportClient = static_cast<FEditorViewportClient*>(PinnedViewport->GetClient()))
			{
				EditorViewportClient->ChannelMaskParams.ColorChannelMask = InChannelMask;
				EditorViewportClient->Invalidate();
			}
		}
	}
	
	virtual void SetAlphaInverted(bool bInAlphaInverted) override
	{
		TWeakPtr<FSceneViewport> Viewport = GetActiveEditorViewport();
		if (TSharedPtr<FSceneViewport> PinnedViewport = Viewport.Pin())
		{
			if (FEditorViewportClient* EditorViewportClient = static_cast<FEditorViewportClient*>(PinnedViewport->GetClient()))
			{
				EditorViewportClient->ChannelMaskParams.bInvertAlphaChannelMask = bInAlphaInverted;
				
				if (!bInAlphaInverted)
				{
					EditorViewportClient->ChannelMaskParams.bDrawAlphaBlendedCheckerboard = false;
				}
				
				EditorViewportClient->Invalidate();
			}
		}
	}

	virtual void SetDrawAlphaBlendedCheckerboard(bool bInDrawAlphaBlendedCheckerboard) override
	{
		TWeakPtr<FSceneViewport> Viewport = GetActiveEditorViewport();
		if (TSharedPtr<FSceneViewport> PinnedViewport = Viewport.Pin())
		{
			if (FEditorViewportClient* EditorViewportClient = static_cast<FEditorViewportClient*>(PinnedViewport->GetClient()))
			{
				EditorViewportClient->ChannelMaskParams.bDrawAlphaBlendedCheckerboard = bInDrawAlphaBlendedCheckerboard;
				EditorViewportClient->Invalidate();
			}
		}
	}

	void SetMediaItemConfigSettings(FMediaProfileEditorPerMediaItemSettings* InMediaItemSettings)
	{
		TWeakPtr<FSceneViewport> Viewport = GetActiveEditorViewport();
		if (TSharedPtr<FSceneViewport> PinnedViewport = Viewport.Pin())
		{
			if (FEditorViewportClient* EditorViewportClient = static_cast<FEditorViewportClient*>(PinnedViewport->GetClient()))
			{
				InMediaItemSettings->ColorChannelMask = (int32)EditorViewportClient->ChannelMaskParams.ColorChannelMask;
				InMediaItemSettings->bInvertAlphaChannelMask = EditorViewportClient->ChannelMaskParams.bInvertAlphaChannelMask;
				InMediaItemSettings->bDrawAlphaBlendedCheckerboard = EditorViewportClient->ChannelMaskParams.bDrawAlphaBlendedCheckerboard;
			}
		}
	}
	
protected:
	virtual TSharedRef<SWidget> GetContentWidget() override
	{
		return SNew(SActiveEditorViewportRenderer)
			.ActiveEditorViewport(this, &SMediaCaptureEditorViewportImage::GetActiveEditorViewport);
	}

private:
	TWeakPtr<FSceneViewport> GetActiveEditorViewport() const
	{
		if (TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin())
		{
			return PinnedMediaProfile->GetPlaybackManager()->GetActiveViewportFromIndex(MediaOutputIndex);
		}

		return nullptr;
	}

	FOptionalSize GetAspectRatio() const
	{
		// If we are actively capturing the active editor viewport, use the supplied aspect ratio
		// Otherwise, use the viewport's desired aspect ratio
		const EMediaCaptureState State = MediaCaptureState.IsSet() ? MediaCaptureState.Get().Get(EMediaCaptureState::Stopped) : EMediaCaptureState::Stopped;
		if (State == EMediaCaptureState::Capturing)
		{
			return AspectRatio.Get(FOptionalSize());
		}

		if (TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin())
		{
			if (TSharedPtr<FSceneViewport> Viewport = PinnedMediaProfile->GetPlaybackManager()->GetActiveViewportFromIndex(MediaOutputIndex))
			{
				return Viewport->GetDesiredAspectRatio();
			}
		}

		return FOptionalSize();
	}
	
private:
	/** The media profile whose media output is being displayed */
	TWeakObjectPtr<UMediaProfile> MediaProfile;

	/** The index of the media output being displayed */
	int32 MediaOutputIndex = INDEX_NONE;
	
	/** Desired aspect ratio of the media output being captured */
	TAttribute<FOptionalSize> AspectRatio;
};

/** Widget to display a viewport media capture output */
class SMediaCaptureViewportImage : public SMediaCaptureImageBase
{
public:
	SLATE_BEGIN_ARGS(SMediaCaptureViewportImage) {}
		SLATE_ATTRIBUTE(TOptional<EMediaCaptureState>, MediaCaptureState)
		SLATE_ATTRIBUTE(FOptionalSize, AspectRatio)
	SLATE_END_ARGS()

	virtual ~SMediaCaptureViewportImage() override
	{
		if (TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin())
		{
			PinnedMediaProfile->GetPlaybackManager()->ReleaseManagedViewportFromIndex(MediaOutputIndex, this);
		}
		
		FEditorDelegates::PrePIEEnded.RemoveAll(this);
		FEditorDelegates::PostPIEStarted.RemoveAll(this);
	}
	
	void Construct(const FArguments& InArgs, UMediaProfile* InMediaProfile, const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo)
	{
		MediaProfile = InMediaProfile;
		MediaOutputIndex = MediaProfile->FindMediaOutputIndex(OutputInfo.MediaOutput);

		IMediaProfileModule& MediaProfileModule = FModuleManager::GetModuleChecked<IMediaProfileModule>("MediaProfile");
		TSharedPtr<FViewportClient> ManagedViewport = InMediaProfile->GetPlaybackManager()->GetOrCreateManagedViewportFromIndex(MediaOutputIndex, OutputInfo.ViewMode, this);
		if (ManagedViewport.IsValid() && MediaProfileModule.CanCaptureWithLevelEditorViewportClients())
		{
			ViewportClient = StaticCastSharedPtr<FLevelEditorViewportClient>(ManagedViewport);
		}

		FEditorDelegates::PostPIEStarted.AddSP(this, &SMediaCaptureViewportImage::OnPostPIEStarted);
		FEditorDelegates::PrePIEEnded.AddSP(this, &SMediaCaptureViewportImage::OnPrePIEEnded);
		
		SMediaCaptureImageBase::FArguments BaseArgs;
		BaseArgs.MediaCaptureState(InArgs._MediaCaptureState);
		BaseArgs.AspectRatio(InArgs._AspectRatio);
		
		SMediaCaptureImageBase::Construct(BaseArgs);

		Refresh(OutputInfo.Cameras);
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (PendingViewportSizeChange.IsSet())
		{
			if (FSceneViewport* SceneViewport = GetSceneViewport())
			{
				const FIntPoint Size = PendingViewportSizeChange.GetValue();
				SceneViewport->SetFixedViewportSize(Size.X, Size.Y);
				PendingViewportSizeChange.Reset();
			}
		}
	}
	
	virtual void SetChannelMask(EColorChannelMask InChannelMask) override
	{
		if (!ViewportClient.IsValid())
		{
			return;
		}
		
		ViewportClient->ChannelMaskParams.ColorChannelMask = InChannelMask;
		ViewportClient->Invalidate();
	}

	virtual void SetAlphaInverted(bool bInAlphaInverted) override
	{
		if (!ViewportClient.IsValid())
		{
			return;
		}
		
		ViewportClient->ChannelMaskParams.bInvertAlphaChannelMask = bInAlphaInverted;

		if (!bInAlphaInverted)
		{
			ViewportClient->ChannelMaskParams.bDrawAlphaBlendedCheckerboard = false;
		}
		
		ViewportClient->Invalidate();
	}

	virtual void SetDrawAlphaBlendedCheckerboard(bool bInDrawAlphaBlendedCheckerboard) override
	{
		if (!ViewportClient.IsValid())
		{
			return;
		}
		
		ViewportClient->ChannelMaskParams.bDrawAlphaBlendedCheckerboard = bInDrawAlphaBlendedCheckerboard;
		ViewportClient->Invalidate();
	}

	/** Updates the display if the capture's locked actors have been changed */
	void Refresh(const TArray<TSoftObjectPtr<AActor>>& InLockedActors)
	{
		LockedActors.Empty();
		LockedActors.Reserve(InLockedActors.Num());
		for (const TSoftObjectPtr<AActor>& ActorRef : InLockedActors)
		{
			AActor* Actor = ActorRef.Get();
			if (Actor)
			{
				LockedActors.Add(Actor);
			}
		}

		if (LockedActors.Num())
		{
			SetLockedActor(0);
		}
		else
		{
			SetLockedActor(INDEX_NONE);
		}
	}

	/** Sets the actor that functions as the camera for the viewport capture */
	void SetLockedActor(int32 InLockedActorIndex)
	{
		CurrentLockedActor = InLockedActorIndex;

		if (!ViewportClient.IsValid())
		{
			return;
		}

		AActor* ActorToLockTo = nullptr;
		if (LockedActors.IsValidIndex(CurrentLockedActor))
		{
			if (!LockedActors[CurrentLockedActor].IsValid())
			{
				// If the locked actor is no longer valid, abort and set the current locked actor to none
				CurrentLockedActor = INDEX_NONE;
				return;
			}
			
			AActor* Actor = LockedActors[CurrentLockedActor].Get();
			ActorToLockTo = Actor;
			
			// If we are in PIE and the locked actor is not, find its PIE counterpart to lock to
			if (GEditor->PlayWorld)
			{
				const bool bIsAlreadyPIEActor = Actor->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
				if (!bIsAlreadyPIEActor)
				{
					ActorToLockTo = EditorUtilities::GetSimWorldCounterpartActor(Actor);
				}
			}
		}

		ViewportClient->SetActorLock(ActorToLockTo);
	}

	/** Queues up a desired viewport size that will be set on the next tick */
	void QueueViewportSizeChange(FIntPoint InDesiredViewportSize)
	{
		PendingViewportSizeChange = InDesiredViewportSize;
	}

	/** Gets the viewport client being displayed by this widget */
	TSharedPtr<FLevelEditorViewportClient> GetViewportClient() const { return ViewportClient; }
	
protected:
	virtual TSharedRef<SWidget> GetContentWidget() override
	{
		if (FSceneViewport* SceneViewport = GetSceneViewport())
		{
			if (TSharedPtr<SViewport> ViewportWidget = SceneViewport->GetViewportWidget().Pin())
			{
				return ViewportWidget.ToSharedRef();
			}
		}
		
		return SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ViewportNotConfiguredLabel", "Viewport output not configured for Media Output Capture"))
			];
	}
	
private:
	FSceneViewport* GetSceneViewport()
	{
		if (!ViewportClient.IsValid())
		{
			return nullptr;
		}

		if (FViewport* Viewport = ViewportClient->Viewport)
		{
			return Viewport->AsSceneViewport();
		}
		
		return nullptr;
	}
	
	/** Raised when Play-in-Editor is started */
	void OnPostPIEStarted(bool bIsSimulating)
	{
		SetLockedActor(CurrentLockedActor);
	}

	/** Raised when Play-in-Editor is ended */
	void OnPrePIEEnded(bool bIsSimulating)
	{
		SetLockedActor(CurrentLockedActor);
	}
	
private:
	/** The media profile whose media output is being displayed */
	TWeakObjectPtr<UMediaProfile> MediaProfile;

	/** The index of the media output being displayed */
	int32 MediaOutputIndex = INDEX_NONE;

	/** The viewport client being used by the media output for capture */
	TSharedPtr<FLevelEditorViewportClient> ViewportClient = nullptr;

	/** List of actors that can function as cameras for the viewport capture */
	TArray<TWeakObjectPtr<AActor>> LockedActors;

	/** The current locked actor that functions as the camera. */
	int32 CurrentLockedActor = INDEX_NONE;

	/**
	 * To manually set the viewport size on the capture, we need the SViewport widget to be fully in the slate hierarchy
	 * (as FSceneViewport needs to be able to find an SWindow from SViewport), so this optional stores any pending size change
	 * that should be applied on the next tick of the widget, which by then should be properly in the window hierarchy
	 */
	TOptional<FIntPoint> PendingViewportSizeChange;
};

/** Widget to display a render target media capture output */
class SMediaCaptureRenderTargetImage : public SMediaCaptureImageBase
{
public:
	SLATE_BEGIN_ARGS(SMediaCaptureRenderTargetImage) {}
		SLATE_ATTRIBUTE(TOptional<EMediaCaptureState>, MediaCaptureState)
		SLATE_ATTRIBUTE(FOptionalSize, AspectRatio)
		SLATE_ATTRIBUTE(EColorChannelMask, ChannelMask)
		SLATE_ATTRIBUTE(bool, InvertAlphaChannel)
		SLATE_ATTRIBUTE(bool, DrawCheckerboard)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UTextureRenderTarget2D* InRenderTarget)
	{
		ChannelMaskAttr = InArgs._ChannelMask;
		InvertAlphaChannelAttr = InArgs._InvertAlphaChannel;
		DrawCheckerboardAttr = InArgs._DrawCheckerboard;
		
		SMediaCaptureImageBase::FArguments BaseArgs;
		BaseArgs.MediaCaptureState(InArgs._MediaCaptureState);
		BaseArgs.AspectRatio(InArgs._AspectRatio);
		
		SMediaCaptureImageBase::Construct(BaseArgs);

		Refresh(InRenderTarget);
	}

	/** Updates the display if the capture's render target has been changed */
	void Refresh(UTextureRenderTarget2D* InRenderTarget)
	{
		if (RenderTarget.IsValid() && RenderTarget.Get() == InRenderTarget)
		{
			return;
		}

		RenderTarget = InRenderTarget;
		if (InRenderTarget)
		{
			Container->SetContent
			(
				SNew(SMaskedMediaImage, InRenderTarget)
				.IsAlphaPremultiplied(true)
				.ChannelMask(ChannelMaskAttr)
				.InvertAlphaChannel(InvertAlphaChannelAttr)
				.DrawCheckerboard(DrawCheckerboardAttr)
			);
		}
		else
		{
			Container->SetContent
			(
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoRenderTargetLabel", "No Render Target set for Media Output Capture"))
				]
			);
		}
	}

protected:
	virtual TSharedRef<SWidget> GetContentWidget() override
	{
		return SAssignNew(Container, SBox);
	}
	
private:
	/** Widget that contains the render target output widget */
	TSharedPtr<SBox> Container;

	/** The render target being used by the capture */
	TWeakObjectPtr<UTextureRenderTarget2D> RenderTarget;
	
	/** Attribute for the color channel to mask to */
	TAttribute<EColorChannelMask> ChannelMaskAttr;

	/** Attribute for the invert alpha flag */
	TAttribute<bool> InvertAlphaChannelAttr;

	/** Attribute for drawing the alpha-blended checkerboard background */
	TAttribute<bool> DrawCheckerboardAttr;
};

/** Widget to display a media texture media capture output */
class SMediaCaptureMediaTextureImage : public SMediaCaptureImageBase
{
public:
	SLATE_BEGIN_ARGS(SMediaCaptureMediaTextureImage) {}
		SLATE_ATTRIBUTE(TOptional<EMediaCaptureState>, MediaCaptureState)
		SLATE_ATTRIBUTE(FOptionalSize, AspectRatio)
		SLATE_ATTRIBUTE(EColorChannelMask, ChannelMask)
		SLATE_ATTRIBUTE(bool, InvertAlphaChannel)
		SLATE_ATTRIBUTE(bool, DrawCheckerboard)
		SLATE_ATTRIBUTE(float, Rotation)
		SLATE_ATTRIBUTE(FVector2f, Scale)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UMediaTexture* InMediaTexture)
	{
		ChannelMaskAttr = InArgs._ChannelMask;
		InvertAlphaChannelAttr = InArgs._InvertAlphaChannel;
		DrawCheckerboardAttr = InArgs._DrawCheckerboard;
		RotationAttr = InArgs._Rotation;
		ScaleAttr = InArgs._Scale;
		
		SMediaCaptureImageBase::FArguments BaseArgs;
		BaseArgs.MediaCaptureState(InArgs._MediaCaptureState);
		BaseArgs.AspectRatio(InArgs._AspectRatio);
		
		SMediaCaptureImageBase::Construct(BaseArgs);

		Refresh(InMediaTexture);
	}

	/** Updates the display if the capture's media texture has been changed */
	void Refresh(UMediaTexture* InMediaTexture)
	{
		if (MediaTexture.IsValid() && MediaTexture.Get() == InMediaTexture)
		{
			return;
		}

		MediaTexture = InMediaTexture;
		if (InMediaTexture)
		{
			Container->SetContent
			(
				SNew(SMaskedMediaImage, InMediaTexture)
				.IsAlphaPremultiplied(true)
				.ChannelMask(ChannelMaskAttr)
				.InvertAlphaChannel(InvertAlphaChannelAttr)
				.DrawCheckerboard(DrawCheckerboardAttr)
				.Rotation(RotationAttr)
				.Scale(ScaleAttr)
			);
		}
		else
		{
			Container->SetContent
			(
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoMediaTextureLabel", "No Media Texture set for Media Output Capture"))
				]
			);
		}
	}

protected:
	virtual TSharedRef<SWidget> GetContentWidget() override
	{
		return SAssignNew(Container, SBox);
	}
	
private:
	/** Widget that contains the render target output widget */
	TSharedPtr<SBox> Container;

	/** The media texture being used by the capture */
	TWeakObjectPtr<UMediaTexture> MediaTexture;
	
	/** Attribute for the color channel to mask to */
	TAttribute<EColorChannelMask> ChannelMaskAttr;

	/** Attribute for the invert alpha flag */
	TAttribute<bool> InvertAlphaChannelAttr;

	/** Attribute for drawing the alpha-blended checkerboard background */
	TAttribute<bool> DrawCheckerboardAttr;
	
	/** Attribute for the rotation to apply to the media texture */
	TAttribute<float> RotationAttr;
	
	/** Attribute for the scale to apply to the media texture */
	TAttribute<FVector2f> ScaleAttr;
};

void SMediaProfileMediaOutputDisplay::Construct(const FArguments& InArgs, const TSharedPtr<SMediaProfileViewport>& InOwningViewport)
{
	ShowFlagsCommandList = MakeShared<FUICommandList>();
	
	SMediaProfileMediaItemDisplayBase::FArguments BaseArgs;
	BaseArgs.MediaProfileEditor(InArgs._MediaProfileEditor);
	BaseArgs.PanelIndex(InArgs._PanelIndex);
	BaseArgs.MediaItemIndex(InArgs._MediaItemIndex);
	SMediaProfileMediaItemDisplayBase::Construct(BaseArgs, InOwningViewport);

	if (MediaProfileEditor.IsValid())
	{
		MediaProfileEditor.Pin()->GetOnCaptureMethodChanged().AddSP(this, &SMediaProfileMediaOutputDisplay::OnCaptureMethodChanged);
	}
}

SMediaProfileMediaOutputDisplay::~SMediaProfileMediaOutputDisplay()
{
	if (MediaProfileEditor.IsValid())
	{
		MediaProfileEditor.Pin()->GetOnCaptureMethodChanged().RemoveAll(this);
	}
}

void SMediaProfileMediaOutputDisplay::ConfigureMediaImage()
{
	ImageWidget.Reset();
	CaptureType = ECaptureType::None;
	CaptureIndex = INDEX_NONE;

	// Clear out all commands that have been bound to any viewport clients from viewport captures
	for (const FShowFlagMenuCommands::FShowFlagCommand& FlagCommand : FShowFlagMenuCommands::Get().GetCommands())
	{
		ShowFlagsCommandList->UnmapAction(FlagCommand.ShowMenuItem);
	}
	bDisplayShowFlags = false;

	UMediaProfile* MediaProfile = GetMediaProfile();
	if (!MediaProfile)
	{
		MediaImageContainer->SetContent(SNullWidget::NullWidget);
		return;
	}
	
	UMediaOutput* MediaOutput = GetMediaItem();
	if (!MediaOutput)
	{
		MediaImageContainer->SetContent(
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(STextBlock).Text(LOCTEXT("MediaOutputNotConfiguredLabel", "Media Output not configured"))
			]);
		return;
	}
	
	if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
	{
		if (CaptureSettings->CurrentViewportMediaOutput.MediaOutput == MediaOutput)
		{
			CaptureType = ECaptureType::ActiveViewport;

			TSharedPtr<SMediaCaptureEditorViewportImage> CurrentViewportImage;
			MediaImageContainer->SetContent(
				SAssignNew(CurrentViewportImage, SMediaCaptureEditorViewportImage, MediaProfile, CaptureSettings->CurrentViewportMediaOutput)
				.MediaCaptureState(this, &SMediaProfileMediaOutputDisplay::GetMediaOutputCaptureState)
				.AspectRatio(this, &SMediaProfileMediaOutputDisplay::GetMediaOutputDesiredAspectRatio)
			);

			ImageWidget = CurrentViewportImage;

			// Instead of overriding with the color channel masking settings from the config, respect the current viewport's settings,
			// and write them to the config instead
			if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
			{
				CurrentViewportImage->SetMediaItemConfigSettings(MediaItemSettings);
			}
			
			return;
		}
		
		CaptureIndex = CaptureSettings->ViewportCaptures.IndexOfByPredicate(
		[MediaOutput](const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo)
			{
				return OutputInfo.MediaOutput == MediaOutput;
			});
		
		if (CaptureSettings->ViewportCaptures.IsValidIndex(CaptureIndex))
		{
			CaptureType = ECaptureType::Viewport;

			TSharedPtr<SMediaCaptureViewportImage> ViewportImage;
			MediaImageContainer->SetContent(
				SAssignNew(ViewportImage, SMediaCaptureViewportImage, MediaProfile, CaptureSettings->ViewportCaptures[CaptureIndex])
				.MediaCaptureState(this, &SMediaProfileMediaOutputDisplay::GetMediaOutputCaptureState)
				.AspectRatio(this, &SMediaProfileMediaOutputDisplay::GetMediaOutputDesiredAspectRatio)
			);

			FIntPoint TargetSize = MediaOutput->GetRequestedSize();
			if (TargetSize == UMediaOutput::RequestCaptureSourceSize)
			{
				TargetSize = GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->DefaultViewportCaptureSize;
			}
			
			ViewportImage->QueueViewportSizeChange(TargetSize);
			ImageWidget = ViewportImage;
			
			if (TSharedPtr<FLevelEditorViewportClient> ViewportClient = ViewportImage->GetViewportClient())
			{
				FShowFlagMenuCommands::Get().BindCommands(*ShowFlagsCommandList.Get(), ViewportClient);
				bDisplayShowFlags = true;
			}

			// Sync the viewport's color channel mask settings with the saved settings in the editor config
			if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
			{
				ViewportImage->SetChannelMask((EColorChannelMask)MediaItemSettings->ColorChannelMask);
				ViewportImage->SetAlphaInverted(MediaItemSettings->bInvertAlphaChannelMask);
				ViewportImage->SetDrawAlphaBlendedCheckerboard(MediaItemSettings->bDrawAlphaBlendedCheckerboard);
			}
			
			return;
		}

		CaptureIndex = CaptureSettings->RenderTargetCaptures.IndexOfByPredicate(
		[MediaOutput](const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& OutputInfo)
			{
				return OutputInfo.MediaOutput == MediaOutput;
			});

		if (CaptureSettings->RenderTargetCaptures.IsValidIndex(CaptureIndex))
		{
			CaptureType = ECaptureType::RenderTarget;

			MediaImageContainer->SetContent(
				SAssignNew(ImageWidget, SMediaCaptureRenderTargetImage, CaptureSettings->RenderTargetCaptures[CaptureIndex].RenderTarget)
				.MediaCaptureState(this, &SMediaProfileMediaOutputDisplay::GetMediaOutputCaptureState)
				.AspectRatio(this, &SMediaProfileMediaOutputDisplay::GetMediaOutputDesiredAspectRatio)
				.ChannelMask(this, &SMediaProfileMediaOutputDisplay::GetChannelMask)
				.InvertAlphaChannel(this, &SMediaProfileMediaOutputDisplay::GetInvertAlphaChannelMask)
				.DrawCheckerboard(this, &SMediaProfileMediaOutputDisplay::GetDrawAlphaBlendedCheckerboard)
			);
			return;
		}

		CaptureIndex = CaptureSettings->MediaTextureCaptures.IndexOfByPredicate(
			[MediaOutput](const FMediaFrameworkCaptureMediaTextureOutputInfo& OutputInfo)
			{
				return OutputInfo.MediaOutput == MediaOutput;
			});

		if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex))
		{
			CaptureType = ECaptureType::MediaTexture;

			MediaImageContainer->SetContent(
				SAssignNew(ImageWidget, SMediaCaptureMediaTextureImage, CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaTexture)
				.MediaCaptureState(this, &SMediaProfileMediaOutputDisplay::GetMediaOutputCaptureState)
				.AspectRatio(this, &SMediaProfileMediaOutputDisplay::GetMediaOutputDesiredAspectRatio)
				.ChannelMask(this, &SMediaProfileMediaOutputDisplay::GetChannelMask)
				.InvertAlphaChannel(this, &SMediaProfileMediaOutputDisplay::GetInvertAlphaChannelMask)
				.DrawCheckerboard(this, &SMediaProfileMediaOutputDisplay::GetDrawAlphaBlendedCheckerboard)
				.Rotation(this, &SMediaProfileMediaOutputDisplay::GetRotationValue, false)
				.Scale(this, &SMediaProfileMediaOutputDisplay::GetScale)
			);
			return;
		}
	}

	MediaImageContainer->SetContent(
		SNew(SBorder)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock).Text(LOCTEXT("MediaCaptureNotConfiguredLabel", "Media capture not configured"))
		]);
}

UMediaOutput* SMediaProfileMediaOutputDisplay::GetMediaItem() const
{
	UMediaProfile* MediaProfile = GetMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}

	return MediaProfile->GetMediaOutput(MediaItemIndex); 
}

FText SMediaProfileMediaOutputDisplay::GetMediaItemLabel() const
{
	UMediaProfile* MediaProfile = GetMediaProfile();
	if (!MediaProfile)
	{
		return FText::GetEmpty();
	}

	return FText::FromString(MediaProfile->GetLabelForMediaOutput(MediaItemIndex));
}

FText SMediaProfileMediaOutputDisplay::GetBaseMediaTypeLabel() const
{
	return LOCTEXT("MediaOutputBaseTypeLabel", "Media Output");
}

bool SMediaProfileMediaOutputDisplay::CanTransformMediaItem() const
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				return true;
			}
		}
	}
	
	return SMediaProfileMediaItemDisplayBase<UMediaOutput>::CanTransformMediaItem();
}

EMediaImageRotation SMediaProfileMediaOutputDisplay::GetRotation()
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				return (EMediaImageRotation)CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.Rotation;
			}
		}
	}
	
	return SMediaProfileMediaItemDisplayBase<UMediaOutput>::GetRotation();
}

ECheckBoxState SMediaProfileMediaOutputDisplay::GetRotationCheckState(EMediaImageRotation InRotation)
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				return (EMediaImageRotation)CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.Rotation == InRotation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
	}
	
	return SMediaProfileMediaItemDisplayBase<UMediaOutput>::GetRotationCheckState(InRotation);
}

float SMediaProfileMediaOutputDisplay::GetRotationValue(bool bInDegrees) const
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				return CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.GetRotationValue(bInDegrees);
			}
		}
	}
	
	return SMediaProfileMediaItemDisplayBase<UMediaOutput>::GetRotationValue(bInDegrees);
}

void SMediaProfileMediaOutputDisplay::SetRotation(EMediaImageRotation InRotation)
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.Rotation = (EMediaCaptureRotation)InRotation;
				if (CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.Rotation != EMediaCaptureRotation::None)
				{
					CaptureSettings->MediaTextureCaptures[CaptureIndex].CaptureOptions.ResizeMethod = EMediaCaptureResizeMethod::ResizeInRenderPass;
				}
				
				if (TSharedPtr<FMediaProfileEditor> PinnedMediaProfileEditor = MediaProfileEditor.Pin())
				{
					PinnedMediaProfileEditor->RestartActiveMediaCaptures(GetMediaItem());
				}
				
				return;
			}
		}
	}
	
	SMediaProfileMediaItemDisplayBase<UMediaOutput>::SetRotation(InRotation);
}

void SMediaProfileMediaOutputDisplay::GetFlipState(bool& bOutFlipHorizontal, bool& bOutFlipVertical)
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				bOutFlipHorizontal = CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipHorizontal;
				bOutFlipVertical = CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipVertical;
				return;
			}
		}
	}
	
	SMediaProfileMediaItemDisplayBase<UMediaOutput>::GetFlipState(bOutFlipHorizontal, bOutFlipVertical);
}

ECheckBoxState SMediaProfileMediaOutputDisplay::GetFlipHorizontalCheckState()
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				return CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipHorizontal ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
	}
	
	return SMediaProfileMediaItemDisplayBase<UMediaOutput>::GetFlipHorizontalCheckState();
}

ECheckBoxState SMediaProfileMediaOutputDisplay::GetFlipVerticalCheckState()
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				return CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipVertical ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
	}
	
	return SMediaProfileMediaItemDisplayBase<UMediaOutput>::GetFlipVerticalCheckState();
}

void SMediaProfileMediaOutputDisplay::ToggleFlipHorizontal()
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipHorizontal = !CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipHorizontal;
				
				if (TSharedPtr<FMediaProfileEditor> PinnedMediaProfileEditor = MediaProfileEditor.Pin())
				{
					PinnedMediaProfileEditor->RestartActiveMediaCaptures(GetMediaItem());
				}
				
				return;
			}
		}
	}
	
	SMediaProfileMediaItemDisplayBase<UMediaOutput>::ToggleFlipHorizontal();
}

void SMediaProfileMediaOutputDisplay::ToggleFlipVertical()
{
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
			{
				CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipVertical = !CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipVertical;
				
				if (TSharedPtr<FMediaProfileEditor> PinnedMediaProfileEditor = MediaProfileEditor.Pin())
				{
					PinnedMediaProfileEditor->RestartActiveMediaCaptures(GetMediaItem());
				}
				
				return;
			}
		}
	}
	
	SMediaProfileMediaItemDisplayBase<UMediaOutput>::ToggleFlipVertical();
}

FVector2f SMediaProfileMediaOutputDisplay::GetScale() const
{
	if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
	{
		if (CaptureType == ECaptureType::MediaTexture &&
			CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
			CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
		{
			return CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.GetScaleValue();
		}
	}
	
	return SMediaProfileMediaItemDisplayBase<UMediaOutput>::GetScale();
}

void SMediaProfileMediaOutputDisplay::ResetTransform()
{
	if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
	{
		if (CaptureType == ECaptureType::MediaTexture &&
			CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
			CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
		{
			CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.Rotation = EMediaCaptureRotation::None;
			CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipHorizontal = false;
			CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipVertical = false;
			
			if (TSharedPtr<FMediaProfileEditor> PinnedMediaProfileEditor = MediaProfileEditor.Pin())
			{
				PinnedMediaProfileEditor->RestartActiveMediaCaptures(GetMediaItem());
			}
			
			return;
		}
	}
	
	SMediaProfileMediaItemDisplayBase<UMediaOutput>::ResetTransform();
}

bool SMediaProfileMediaOutputDisplay::CanResetTransform() const
{
	if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
	{
		if (CaptureType == ECaptureType::MediaTexture &&
			CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
			CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem())
		{
			return
				CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.Rotation != EMediaCaptureRotation::None ||
				CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipHorizontal ||
				CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.bFlipVertical;
		}
	}
	
	return false;
}

void SMediaProfileMediaOutputDisplay::SetChannelMask(EColorChannelMask InChannelMask)
{
	SMediaProfileMediaItemDisplayBase::SetChannelMask(InChannelMask);
	
	if (TSharedPtr<SMediaCaptureImageBase> CaptureImageWidget = StaticCastSharedPtr<SMediaCaptureImageBase>(ImageWidget))
    {
    	CaptureImageWidget->SetChannelMask(InChannelMask);
    }
}

void SMediaProfileMediaOutputDisplay::SetInvertAlphaChannelMask(bool bInInvertAlphaChannelMask)
{
	SMediaProfileMediaItemDisplayBase::SetInvertAlphaChannelMask(bInInvertAlphaChannelMask);
	
	if (TSharedPtr<SMediaCaptureImageBase> CaptureImageWidget = StaticCastSharedPtr<SMediaCaptureImageBase>(ImageWidget))
	{
		CaptureImageWidget->SetAlphaInverted(bInInvertAlphaChannelMask);
	}
}

void SMediaProfileMediaOutputDisplay::SetDrawAlphaBlendedCheckerboard(bool bInDrawAlphaBlendedCheckerboard)
{
	SMediaProfileMediaItemDisplayBase::SetDrawAlphaBlendedCheckerboard(bInDrawAlphaBlendedCheckerboard);
	
	if (TSharedPtr<SMediaCaptureImageBase> CaptureImageWidget = StaticCastSharedPtr<SMediaCaptureImageBase>(ImageWidget))
	{
		return CaptureImageWidget->SetDrawAlphaBlendedCheckerboard(bInDrawAlphaBlendedCheckerboard);
	}
}

bool SMediaProfileMediaOutputDisplay::IsDrawAlphaBlendedCheckerboardEnabled() const
{
	// UE-354997: Don't allow alpha blended checkerboards for media outputs for now as they can inadvertently be passed to the media output stream
	return false;
}

bool SMediaProfileMediaOutputDisplay::CanShowAlphaCheckerboard() const
{
	// UE-354997: Don't allow alpha blended checkerboards for media outputs for now as they can inadvertently be passed to the media output stream
	return false;
}

void SMediaProfileMediaOutputDisplay::AddToolbarEntries(FToolMenuSection& Section)
{
	FToolMenuEntry& ShowSubmenuEntry = Section.AddEntry(UE::UnrealEd::CreateShowSubmenu(
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* Submenu)
		{
			FShowFlagMenuCommands::Get().BuildShowFlagsMenu(Submenu);
		})));

	ShowSubmenuEntry.Visibility = TAttribute<bool>::CreateLambda([this] { return bDisplayShowFlags; });
}

void SMediaProfileMediaOutputDisplay::AppendToolbarCommandList(FToolMenuContext& Context)
{
	Context.AppendCommandList(ShowFlagsCommandList);
}

void SMediaProfileMediaOutputDisplay::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	SMediaProfileMediaItemDisplayBase::OnObjectPropertyChanged(InObject, InPropertyChangedEvent);
	
	if (UMediaProfileEditorCaptureSettings* CaptureSettings = Cast<UMediaProfileEditorCaptureSettings>(InObject))
	{
		if (CaptureSettings != FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			return;
		}

		const FName PropertyName = InPropertyChangedEvent.GetPropertyName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FMediaFrameworkCaptureRenderTargetCameraOutputInfo, RenderTarget))
		{
			if (CaptureType != ECaptureType::RenderTarget || !CaptureSettings->RenderTargetCaptures.IsValidIndex(CaptureIndex))
			{
				return;
			}
	
			TSharedPtr<SMediaCaptureRenderTargetImage> RenderTargetImageWidget = StaticCastSharedPtr<SMediaCaptureRenderTargetImage>(ImageWidget);
			if (!RenderTargetImageWidget.IsValid())
			{
				return;
			}

			RenderTargetImageWidget->Refresh(CaptureSettings->RenderTargetCaptures[CaptureIndex].RenderTarget);
			return;
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FMediaFrameworkCaptureCameraViewportCameraOutputInfo, Cameras))
		{
			if (CaptureType != ECaptureType::Viewport || !CaptureSettings->ViewportCaptures.IsValidIndex(CaptureIndex))
			{
				return;
			}
			
			TSharedPtr<SMediaCaptureViewportImage> ViewportImageWidget = StaticCastSharedPtr<SMediaCaptureViewportImage>(ImageWidget);
			if (!ViewportImageWidget.IsValid())
			{
				return;
			}

			ViewportImageWidget->Refresh(CaptureSettings->ViewportCaptures[CaptureIndex].Cameras);
			return;
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FMediaFrameworkCaptureMediaTextureOutputInfo, MediaTexture))
		{
			if (CaptureType != ECaptureType::MediaTexture || !CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex))
			{
				return;
			}
	
			TSharedPtr<SMediaCaptureMediaTextureImage> MediaTextureImageWidget = StaticCastSharedPtr<SMediaCaptureMediaTextureImage>(ImageWidget);
			if (!MediaTextureImageWidget.IsValid())
			{
				return;
			}

			MediaTextureImageWidget->Refresh(CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaTexture);
			return;
		}
		
		// If a media output property in the capture settings has changed, or the viewport captures or render target captures lists have changed,
		// check to see the media output object for this display is still linked to a valid capture. If not, refresh the widget
		if (PropertyName == TEXT("MediaOutput") ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMediaFrameworkWorldSettingsAssetUserData, ViewportCaptures) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMediaFrameworkWorldSettingsAssetUserData, RenderTargetCaptures) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMediaFrameworkWorldSettingsAssetUserData, MediaTextureCaptures))
		{
			bool bResetMediaImage = false;
			if (CaptureType == ECaptureType::Viewport)
			{
				bResetMediaImage = !CaptureSettings->ViewportCaptures.IsValidIndex(CaptureIndex) ||
					CaptureSettings->ViewportCaptures[CaptureIndex].MediaOutput != GetMediaItem();
			}
			else if (CaptureType == ECaptureType::RenderTarget)
			{
				bResetMediaImage = !CaptureSettings->RenderTargetCaptures.IsValidIndex(CaptureIndex) ||
					CaptureSettings->RenderTargetCaptures[CaptureIndex].MediaOutput != GetMediaItem();
			}
			else if (CaptureType == ECaptureType::MediaTexture)
			{
				bResetMediaImage = !CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) ||
					CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput != GetMediaItem();
			}
			else if (CaptureType == ECaptureType::ActiveViewport)
			{
				bResetMediaImage = CaptureSettings->CurrentViewportMediaOutput.MediaOutput != GetMediaItem();
			}

			if (bResetMediaImage)
			{
				ConfigureMediaImage();
			}
		}
	}
}

TOptional<EMediaCaptureState> SMediaProfileMediaOutputDisplay::GetMediaOutputCaptureState() const
{
	UMediaProfile* MediaProfile = GetMediaProfile();
	if (!MediaProfile)
	{
		return TOptional<EMediaCaptureState>();
	}

	bool bHasError = false;
	TOptional<EMediaCaptureState> CaptureState = MediaProfile->GetPlaybackManager()->GetOutputCaptureStateFromIndex(MediaItemIndex, bHasError);

	return bHasError ? EMediaCaptureState::Error : CaptureState;
}

FOptionalSize SMediaProfileMediaOutputDisplay::GetMediaOutputDesiredAspectRatio() const
{
	UMediaOutput* MediaOutput = GetMediaItem();
	if (!MediaOutput)
	{
		return FOptionalSize();
	}
	
	FIntPoint TargetSize = MediaOutput->GetRequestedSize();
	if (TargetSize == UMediaOutput::RequestCaptureSourceSize)
	{
		TargetSize = GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->DefaultViewportCaptureSize;
	}
	
	if (CaptureType == ECaptureType::MediaTexture)
	{
		if (UMediaProfileEditorCaptureSettings* CaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings())
		{
			if (CaptureSettings->MediaTextureCaptures.IsValidIndex(CaptureIndex) &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaOutput == GetMediaItem() &&
				CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaTexture)
			{
				// For media texture, _always_ preserve aspect ratio even in situations when other capture methods would stretch (e.g. Resize in Render Pass) to ensure
				// that the preview display of the capture matches what is displayed in the actual output
				FIntPoint MediaTextureSize = FIntPoint(
					CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaTexture->GetWidth(),
					CaptureSettings->MediaTextureCaptures[CaptureIndex].MediaTexture->GetHeight());

				float Rotation = CaptureSettings->MediaTextureCaptures[CaptureIndex].Transform.GetRotationValue();

				const FVector2D RotatedSize = FVector2D(
					MediaTextureSize.X * FMath::Abs(FMath::Cos(Rotation)) + MediaTextureSize.Y * FMath::Abs(FMath::Sin(Rotation)),
					MediaTextureSize.X * FMath::Abs(FMath::Sin(Rotation)) + MediaTextureSize.Y * FMath::Abs(FMath::Cos(Rotation)));

				TargetSize = FIntPoint(FMath::CeilToInt(RotatedSize.X), FMath::CeilToInt(RotatedSize.Y));
			}
		}
	}
	
	const float AspectRatio = (TargetSize.X > 0 && TargetSize.Y > 0) ? (float)TargetSize.X / (float)TargetSize.Y : 1.77777777f;
	return AspectRatio;
}

void SMediaProfileMediaOutputDisplay::OnCaptureMethodChanged(UMediaOutput* MediaOutput)
{
	if (MediaOutput != GetMediaItem())
	{
		return;
	}

	ConfigureMediaImage();
}

void SMediaProfileDummyDisplay::Construct(const FArguments& InArgs, const TSharedPtr<SMediaProfileViewport>& InOwningViewport)
{
	SMediaProfileMediaItemDisplayBase::FArguments BaseArgs;
	BaseArgs.MediaProfileEditor(InArgs._MediaProfileEditor);
	BaseArgs.PanelIndex(InArgs._PanelIndex);
	BaseArgs.MediaItemIndex(INDEX_NONE);
	SMediaProfileMediaItemDisplayBase::Construct(BaseArgs, InOwningViewport);
}

void SMediaProfileDummyDisplay::ConfigureMediaImage()
{
	MediaImageContainer->SetContent(
		SNew(SBorder)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoMediaItemSelectedLabel", "No Media Item selected"))
		]
	);
}

FText SMediaProfileDummyDisplay::GetActiveMediaItemLabel() const
{
	return LOCTEXT("NoMediaItemSelectedLabel", "No Media Item selected");
}

void SMediaProfileDummyDisplay::ChangeActiveMediaItem(UClass* InMediaItemClass, int32 InMediaItemIndex)
{
	constexpr bool bRefreshDisplay = true;
	ChangePanelContents(InMediaItemClass, InMediaItemIndex, bRefreshDisplay);
}

#undef LOCTEXT_NAMESPACE
