// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportCommands.h"
#include "LevelViewportActions.h"
#include "MediaOutput.h"
#include "MediaProfileEditor.h"
#include "MediaProfileEditorUserSettings.h"
#include "MediaSource.h"
#include "PropertyEditorModule.h"
#include "SceneView.h"
#include "SMediaProfileViewport.h"
#include "ToolMenus.h"
#include "ToolMenusEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Profile/MediaProfile.h"
#include "Styling/SlateIconFinder.h"
#include "Tests/ToolMenusTestUtilities.h"
#include "UI/MediaFrameworkTimecodeGenlockToolMenuEntry.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"

#include "SMediaProfileMediaItemDisplay.generated.h"

class SMaskedMediaImage;
class UMediaTexture;
class UMediaProfile;
struct FMediaTextureTrackerObject;

namespace UE::MediaPlayerEditor
{
	class FViewportTileVisibilityProvider;
}

/**
 * Used as the context object for the viewport toolbar displayed at the top of each display, stores a reference to the viewport widget
 * that is displaying the toolbar instance
 */
UCLASS()
class UMediaProfileMediaItemDisplayContext : public UObject
{
	GENERATED_BODY()

public:
	template<typename T>
	TSharedPtr<T> GetDisplayWidget() const
	{
		if (DisplayWidget.IsValid())
		{
			return StaticCastSharedPtr<T>(DisplayWidget.Pin());
		}

		return nullptr;
	}
	
	template<typename T>
	void SetDisplayWidget(const TSharedPtr<T>& InDisplayWidget)
	{
		DisplayWidget = InDisplayWidget;
	}
	
private:
	TWeakPtr<SWidget> DisplayWidget;
};

#define LOCTEXT_NAMESPACE "SMediaProfileMediaItemDisplay"

/** Widget to display the input or capture from a single media source or output */
template<typename TMediaItem>
class SMediaProfileMediaItemDisplayBase : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaProfileMediaItemDisplayBase) {}
		SLATE_ARGUMENT(TWeakPtr<FMediaProfileEditor>, MediaProfileEditor)
		SLATE_ARGUMENT(int32, PanelIndex)
		SLATE_ARGUMENT(int32, MediaItemIndex)
	SLATE_END_ARGS()
		
	void Construct(const FArguments& InArgs, const TSharedPtr<SMediaProfileViewport>& InOwningViewport)
	{
		OwningViewport = InOwningViewport;
		MediaProfileEditor = InArgs._MediaProfileEditor;
		MediaItemIndex = InArgs._MediaItemIndex;
		PanelIndex = InArgs._PanelIndex;

		TimecodeGenlockToolMenuEntry = MakeShared<FMediaFrameworkTimecodeGenlockToolMenuEntry>(GetMediaProfile());
		
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SMediaProfileMediaItemDisplayBase::OnObjectPropertyChanged);

		CommandList = MakeShared<FUICommandList>();
		BindCommands();
		
		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						CreateToolbar()
					]
				]
			]

			+SVerticalBox::Slot()
			[
				SAssignNew(Overlay, SOverlay)

				+SOverlay::Slot()
				[
					SAssignNew(MediaImageContainer, SBorder)
					.BorderBackgroundColor(FLinearColor::Black)
					.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				]
			]
		];

		ConfigureMediaImage();
	}

	virtual ~SMediaProfileMediaItemDisplayBase() override
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	virtual bool SupportsKeyboardFocus() const override { return true; }
	
protected:
	/** Performs any needed configuration to output the media item to the UI */
	virtual void ConfigureMediaImage() = 0;

	/** Gets the typed media item being displayed */
	virtual TMediaItem* GetMediaItem() const = 0;

	/** Gets the label to display for the media item */
	virtual FText GetMediaItemLabel() const = 0;

	/** Gets the label for the base type of the media item (e.g. media source/media output) */
	virtual FText GetBaseMediaTypeLabel() const = 0;

	UMediaProfile* GetMediaProfile() const
	{
		UMediaProfile* MediaProfile = nullptr;
		if (MediaProfileEditor.IsValid())
		{
			MediaProfile = MediaProfileEditor.Pin()->GetMediaProfile();
		}
		
		return MediaProfile;
	}
	
	/** Allows implementations to add their own entries to the viewport toolbar */
	virtual void AddToolbarEntries(FToolMenuSection& Section) { }

	/** Allows implementations to add their own command lists to the viewport toolbar */
	virtual void AppendToolbarCommandList(FToolMenuContext& Context) { }

	/** Binds commands to the display's command list */
	void BindCommands()
	{
		CommandList->MapAction(
			FLevelViewportCommands::Get().ToggleImmersive,
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					TSharedPtr<SMediaProfileViewport> PinnedOwningViewport = OwningViewport.Pin();
					if (!PinnedOwningViewport.IsValid())
					{
						return;
					}
						
					if (!IsImmersive())
					{
						PinnedOwningViewport->SetImmersivePanel();
					}
					else
					{
						PinnedOwningViewport->ClearImmersivePanel();
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this] { return IsImmersive(); })
			));

		CommandList->MapAction(FLevelViewportCommands::Get().ToggleMaximize,
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					TSharedPtr<SMediaProfileViewport> PinnedOwningViewport = OwningViewport.Pin();
					if (!PinnedOwningViewport.IsValid())
					{
						return;
					}
						
					if (!IsMaximized())
					{
						PinnedOwningViewport->MaximizePanel(PanelIndex);
					}
					else
					{
						PinnedOwningViewport->RestorePreviousLayout();
					}
				}),
				FCanExecuteAction::CreateLambda([this]()
				{
					TSharedPtr<SMediaProfileViewport> PinnedOwningViewport = OwningViewport.Pin();
					if (!PinnedOwningViewport.IsValid())
					{
						return false;
					}

					return IsMaximized() ? PinnedOwningViewport->CanRestorePreviousLayout() : PinnedOwningViewport->CanMaximizePanel(PanelIndex);
				})
			));

		CommandList->MapAction(FEditorViewportCommands::Get().AllChannelsMask,
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::SetChannelMask, EColorChannelMask::All),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMediaProfileMediaItemDisplayBase::IsChannelBeingMasked, EColorChannelMask::All)
			));
		
		CommandList->MapAction(FEditorViewportCommands::Get().RedChannelMask,
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::SetChannelMask, EColorChannelMask::Red),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMediaProfileMediaItemDisplayBase::IsChannelBeingMasked, EColorChannelMask::Red)
			));

		CommandList->MapAction(FEditorViewportCommands::Get().GreenChannelMask,
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::SetChannelMask, EColorChannelMask::Green),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMediaProfileMediaItemDisplayBase::IsChannelBeingMasked, EColorChannelMask::Green)
			));

		CommandList->MapAction(FEditorViewportCommands::Get().BlueChannelMask,
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::SetChannelMask, EColorChannelMask::Blue),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMediaProfileMediaItemDisplayBase::IsChannelBeingMasked, EColorChannelMask::Blue)
			));

		CommandList->MapAction(FEditorViewportCommands::Get().AlphaChannelMask,
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::SetChannelMask, EColorChannelMask::Alpha),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMediaProfileMediaItemDisplayBase::IsChannelBeingMasked, EColorChannelMask::Alpha)
			));
	}

	/** Creates the viewport display's toolbar */
	TSharedRef<SWidget> CreateToolbar()
	{
		const FName ViewportToolbarName = "MediaProfileEditor.ViewportToolbar";

		if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
		{
			UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(ViewportToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
			ToolbarMenu->StyleName = "ViewportToolbar";

			// Need empty left section to ensure the middle and right sections get positioned correctly
			FToolMenuSection& LeftSection = ToolbarMenu->AddSection("Left");
			LeftSection.Alignment = EToolMenuSectionAlign::First;
			
			LeftSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				UMediaProfileMediaItemDisplayContext* Context = Section.FindContext<UMediaProfileMediaItemDisplayContext>();
				TSharedPtr<SMediaProfileMediaItemDisplayBase<TMediaItem>> ContextWidget = Context->GetDisplayWidget<SMediaProfileMediaItemDisplayBase<TMediaItem>>();
				
				if (!ContextWidget.IsValid())
				{
					return;
				}

				FToolMenuEntry TimecodeEntry = ContextWidget->TimecodeGenlockToolMenuEntry->CreateTimecodeToolMenuEntry(TAttribute<bool>::CreateLambda([]
				{
					if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
					{
						return UserSettings->bShowTimecodeInViewportToolbar;
					}

					return true;
				}));

				//TODO: Hidden for now until per-source timecode can be figured out (see UE-UE-305891)
				//Section.AddEntry(TimecodeEntry);
				
				FToolMenuEntry GenlockEntry = ContextWidget->TimecodeGenlockToolMenuEntry->CreateGenlockToolMenuEntry(TAttribute<bool>::CreateLambda([]
				{
					if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
					{
						return UserSettings->bShowGenlockInViewportToolbar;
					}

					return true;
				}));

				//TODO: Hidden for now until per-source genlock can be figured out (see UE-UE-305891)
				//Section.AddEntry(GenlockEntry);
			}));
			
			FToolMenuSection& RightSection = ToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			RightSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				UMediaProfileMediaItemDisplayContext* Context = Section.FindContext<UMediaProfileMediaItemDisplayContext>();
				TSharedPtr<SMediaProfileMediaItemDisplayBase<TMediaItem>> ContextWidget = Context->GetDisplayWidget<SMediaProfileMediaItemDisplayBase<TMediaItem>>();
				
				if (!ContextWidget.IsValid())
				{
					return;
				}
				
				Section.AddEntry(ContextWidget->CreateMediaItemsComboButton());
				Section.AddEntry(ContextWidget->CreateChannelsButton());
				ContextWidget->AddToolbarEntries(Section);
				Section.AddEntry(ContextWidget->CreateViewportArrangementMenu());

				//TODO: Hidden for now until per-source timecode/genlock can be figured out (see UE-UE-305891)
				//Section.AddEntry(ContextWidget->CreateSettingsMenu());
			}));
		}
		
		FToolMenuContext Context;
		{
			Context.AppendCommandList(CommandList);
			AppendToolbarCommandList(Context);
			
			UMediaProfileMediaItemDisplayContext* ContextObject = NewObject<UMediaProfileMediaItemDisplayContext>();
			ContextObject->SetDisplayWidget<SMediaProfileMediaItemDisplayBase<TMediaItem>>(SharedThis(this));
			Context.AddObject(ContextObject);
		}

		return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, Context);
	}

	/** Creates the combo button used to change which media item is being displayed in the current panel */
	FToolMenuEntry CreateMediaItemsComboButton()
	{
		return FToolMenuEntry::InitSubMenu(
			TEXT("ActiveMediaItem"),
			TAttribute<FText>::CreateSP(this, &SMediaProfileMediaItemDisplayBase::GetActiveMediaItemLabel),
			TAttribute<FText>(),
			FNewToolMenuDelegate::CreateSP(this, &SMediaProfileMediaItemDisplayBase::GetMediaItemsDropdownContent),
			false,
			TAttribute<FSlateIcon>::CreateSP(this, &SMediaProfileMediaItemDisplayBase::GetActiveMediaItemIcon));	
	}

	/** Creates the dropdown list for the media items combo box, which lists all possible media items that can be displayed in the panel */
	void GetMediaItemsDropdownContent(UToolMenu* ToolMenu)
	{
		UMediaProfile* MediaProfile = GetMediaProfile();

		FToolMenuSection& MediaSourcesSection = ToolMenu->AddSection(TEXT("MediaSourcesSection"), LOCTEXT("MediaSourcesSectionLabel", "Media Sources"));
		for (int32 Index = 0; Index < MediaProfile->NumMediaSources(); ++Index)
		{
			UMediaSource* MediaSource = MediaProfile->GetMediaSource(Index);
			UClass* Class = MediaSource ? MediaSource->GetClass() : UMediaSource::StaticClass();
				
			const FText MediaSourceLabel = MediaSource ? FText::FromString(MediaProfile->GetLabelForMediaSource(Index)) : LOCTEXT("NoMediaSourceLabel", "No Media Source Set");

			MediaSourcesSection.AddMenuEntry(
				FName(FString::Format(TEXT("MediaSourceEntry_{0}"), { Index })),
				FText::Format(LOCTEXT("MediaItemMenuEntryFormat", "{0}: {1}"), Index + 1, MediaSourceLabel),
				TAttribute<FText>(),
				MediaSource ? FSlateIconFinder::FindIconForClass(Class) : FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase<TMediaItem>::ChangeActiveMediaItem, Class, Index),
					FCanExecuteAction::CreateLambda([bIsMediaSourceSet = MediaSource != nullptr]()
					{
						return bIsMediaSourceSet;
					})));
		}

		FToolMenuSection& MediaOutputsSection = ToolMenu->AddSection(TEXT("MediaOutputsSection"), LOCTEXT("MediaOutputsSectionLabel", "Media Outputs"));
		for (int32 Index = 0; Index < MediaProfile->NumMediaSources(); ++Index)
		{
			UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(Index);
			UClass* Class = MediaOutput ? MediaOutput->GetClass() : UMediaOutput::StaticClass(); 

			const FText MediaOutputLabel = MediaOutput ? FText::FromString(MediaProfile->GetLabelForMediaOutput(Index)) : LOCTEXT("NoMediaOutputLabel", "No Media Output Set");
			MediaOutputsSection.AddMenuEntry(
				FName(FString::Format(TEXT("MediaOutputEntry_{0}"), { Index })),
				FText::Format(LOCTEXT("MediaItemMenuEntryFormat", "{0}: {1}"), Index + 1, MediaOutputLabel),
				TAttribute<FText>(),
				MediaOutput ? FSlateIconFinder::FindIconForClass(Class) : FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase<TMediaItem>::ChangeActiveMediaItem, Class, Index),
					FCanExecuteAction::CreateLambda([bIsMediaOutputSet = MediaOutput != nullptr]()
					{
						return bIsMediaOutputSet;
					}))
				);
		}

		FToolMenuSection& LockPanelSection = ToolMenu->AddSection(TEXT("LockPanelSection"));
		LockPanelSection.AddSeparator(NAME_None);
		
		FToolMenuEntry& LockButton = LockPanelSection.AddMenuEntry(
			TEXT("LockDisplayButton"),
			LOCTEXT("LockDisplayLabel", "Lock Display"),
			LOCTEXT("LockDisplayTooltip", "Locks this display to the current media item"),
			TAttribute<FSlateIcon>::CreateSP(this, &SMediaProfileMediaItemDisplayBase::GetLockIcon),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::OnPanelLockToggled),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMediaProfileMediaItemDisplayBase::IsPanelLocked)),
			EUserInterfaceActionType::ToggleButton);
			
		LockButton.SetShowInToolbarTopLevel(true);
		LockButton.ToolBarData.ResizeParams.AllowClipping = false;

		FToolMenuSection& TransformSection = ToolMenu->AddSection(TEXT("TransformSection"), LOCTEXT("TransformSectionLabel", "Transform"));
		TransformSection.AddEntry(CreateRotationSubmenu());
		TransformSection.AddEntry(CreateFlipSubmenu());
		TransformSection.AddMenuEntry(
			"ResetTransform",
			LOCTEXT("ResetTransformLabel", "Reset Transform"),
			LOCTEXT("ResetTransformToolTip", "Resets all transforms on the media item"),
			 FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Undo"),
			 FUIAction(
			 	FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::ResetTransform),
			 	FCanExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::CanResetTransform)));
		
		FToolMenuSection& ChannelsSection = ToolMenu->AddSection(TEXT("ChannelsSection"), LOCTEXT("ChannelsSectionLabel", "Channels"));
		ChannelsSection.AddEntry(CreateChannelsSubmenu(true));
	}
	
	FToolMenuEntry CreateRotationSubmenu()
	{
		FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
			"Rotation",
			TAttribute<FText>::CreateLambda([this]
			{
				const EMediaImageRotation Rotation = GetRotation();
				return FText::Format(LOCTEXT("RotationMenu_LabelFormat", "Rotation ({0})"), StaticEnum<EMediaImageRotation>()->GetDisplayNameTextByValue((int64)Rotation));
			}),
			LOCTEXT("RotationMenu_ToolTip", "The rotation of the media item in the viewport"),
			FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
			{
				FToolMenuSection& RotationSubMenuSection =
					InMenu->AddSection("RotationSubMenuSection", LOCTEXT("RotationSubMenuSectionHeader", "Rotation"));
				
				auto CreateRotationMenuEntry = [this, &RotationSubMenuSection](EMediaImageRotation InRotation)
				{
					RotationSubMenuSection.AddMenuEntry(
						"",
						StaticEnum<EMediaImageRotation>()->GetDisplayNameTextByValue((int64)InRotation),
						FText::GetEmpty(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::SetRotation, InRotation),
							FCanExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::CanTransformMediaItem),
							FGetActionCheckState::CreateSP(this, &SMediaProfileMediaItemDisplayBase::GetRotationCheckState, InRotation)
						),
						EUserInterfaceActionType::Check
					);
				};
				
				CreateRotationMenuEntry(EMediaImageRotation::CCW90);
				CreateRotationMenuEntry(EMediaImageRotation::None);
				CreateRotationMenuEntry(EMediaImageRotation::CW90);
				CreateRotationMenuEntry(EMediaImageRotation::CW180);
			}),
			/*bInOpenSubMenuOnClick=*/false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.RotateMode"),
			/*bInShouldCloseWindowAfterMenuSelection=*/false
		);

		return Entry;
	}
	
	FToolMenuEntry CreateFlipSubmenu()
	{
		FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
			"Flip",
			TAttribute<FText>::CreateLambda([this]
			{
				bool bFlipHorizontal, bFlipVertical;
				GetFlipState(bFlipHorizontal, bFlipVertical);
				FText FlipText = LOCTEXT("FlipNone", "None");
				if (bFlipHorizontal)
				{
					if (bFlipVertical)
					{
						FlipText = LOCTEXT("FlipBoth", "Both");
					}
					else
					{
						FlipText = LOCTEXT("FlipHorizontal", "Horizontal");
					}
				}
				else if (bFlipVertical)
				{
					FlipText = LOCTEXT("FlipVertical", "Vertical");
				}
				
				return FText::Format(LOCTEXT("FlipMenu_LabelFormat", "Flip ({0})"), FlipText);
			}),
			LOCTEXT("FlipMenu_ToolTip", "The horizontal and vertical flip of the media item in the viewport"),
			FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
			{
				FToolMenuSection& FlipSubMenuSection =
					InMenu->AddSection("FlipSubMenuSection", LOCTEXT("FlipSubMenuSectionHeader", "Flip"));

				FlipSubMenuSection.AddMenuEntry(
					"FlipHorizontal",
					LOCTEXT("FlipHorizontalEntry", "Flip Horizontal"),
					FText::GetEmpty(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FlipHorizontal"),
					FUIAction(
						FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::ToggleFlipHorizontal),
						FCanExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::CanTransformMediaItem),
						FGetActionCheckState::CreateSP(this, &SMediaProfileMediaItemDisplayBase::GetFlipHorizontalCheckState)
					),
					EUserInterfaceActionType::Check
				);
				
				FlipSubMenuSection.AddMenuEntry(
					"FlipVertical",
					LOCTEXT("FlipVerticalEntry", "Flip Vertical"),
					FText::GetEmpty(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FlipVertical"),
					FUIAction(
						FExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::ToggleFlipVertical),
						FCanExecuteAction::CreateSP(this, &SMediaProfileMediaItemDisplayBase::CanTransformMediaItem),
						FGetActionCheckState::CreateSP(this, &SMediaProfileMediaItemDisplayBase::GetFlipVerticalCheckState)
					),
					EUserInterfaceActionType::Check
				);
			}),
			/*bInOpenSubMenuOnClick=*/false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FlipVertical"),
			/*bInShouldCloseWindowAfterMenuSelection=*/false
		);

		return Entry;
	}
	
	FToolMenuEntry CreateChannelsSubmenu(bool bUseFullLabel)
	{
		static FText ChannelLabels[] =
		{
			LOCTEXT("ChannelRGB_Label", "RGB"),
			LOCTEXT("ChannelRed_Label", "Red"),
			LOCTEXT("ChannelGreen_Label", "Green"),
			LOCTEXT("ChannelBlue_Label", "Blue"),
			LOCTEXT("ChannelAlpha_Label", "Alpha")
		};

		static FName ChannelIcons[] =
		{
			"EditorViewport.AllChannelsMask",
			"EditorViewport.RedChannelMask",
			"EditorViewport.GreenChannelMask",
			"EditorViewport.BlueChannelMask",
			"EditorViewport.AlphaChannelMask"
		};

		FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
			"Channels",
			TAttribute<FText>::CreateLambda([this, bUseFullLabel]
			{
				const int8 ChannelMaskIndex = (int8)GetChannelMask();
				return bUseFullLabel
					? FText::Format(LOCTEXT("ChannelsMenu_LabelFormat", "Channels ({0})"), ChannelLabels[ChannelMaskIndex + 1])
					: ChannelLabels[ChannelMaskIndex + 1];
			}),
			LOCTEXT("ChannelsMenu_ToolTip", "Display specific color channels in this media source/output"),
			FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
			{
				FToolMenuSection& ChannelsSubMenuSection =
					InMenu->AddSection("ChannelsSubMenuSection", LOCTEXT("ChannelsSubMenuSectionHeader", "Channels"));

				ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().AllChannelsMask);
				ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().RedChannelMask);
				ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().GreenChannelMask);
				ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().BlueChannelMask);
				ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().AlphaChannelMask);

				FToolMenuSection& OptionsSubMenuSection =
					InMenu->AddSection("OptionsSubMenuSection", LOCTEXT("OptionsSubMenuSectionHeader", "Options"));

				OptionsSubMenuSection.AddMenuEntry(
					"ShowChannelsInViewportToolbar",
					LOCTEXT("ShowChannelsInToolbarLabel", "Show Channels in Viewport Toolbar"),
					LOCTEXT("ShowChannelsInToolbarToolTip", "Show the channel selector in the viewport toolbar"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([]()
						{
							if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
							{
								UserSettings->bShowChannelsInViewportToolbar = !UserSettings->bShowChannelsInViewportToolbar;
							}
						}),
						FCanExecuteAction(),
						FGetActionCheckState::CreateLambda([]()
						{
							return GetDefault<UMediaProfileEditorUserSettings>()->bShowChannelsInViewportToolbar ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					),
					EUserInterfaceActionType::ToggleButton
				);

				FToolMenuEntry& InvertAlphaEntry = OptionsSubMenuSection.AddMenuEntry(
					"InvertAlpha",
					LOCTEXT("InvertAlphaLabel", "View Alpha as Inverted"),
					LOCTEXT("InvertAlphaToolTip", "Invert the alpha value displayed when viewing the alpha channel"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this]
						{
							SetInvertAlphaChannelMask(!GetInvertAlphaChannelMask());
						}),
						FCanExecuteAction(),
						FGetActionCheckState::CreateLambda([this]
						{
							return GetInvertAlphaChannelMask() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					),
					EUserInterfaceActionType::ToggleButton
				);
				InvertAlphaEntry.Visibility = TAttribute<bool>::CreateSP(this, &SMediaProfileMediaItemDisplayBase::CanInvertAlphaChannelMask);

				FToolMenuEntry& AlphaCheckerboardEntry = OptionsSubMenuSection.AddMenuEntry(
					"VisualizeAlphaWithCheckerboard",
					LOCTEXT("VisualizeWithCheckerboardLabel", "Visualize Alpha with Checkerboard"),
					LOCTEXT("VisualizeWithCheckerboardToolTip", "Displays a checkerboard underneath the viewport to help visualize the alpha channel"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this]
						{
							SetDrawAlphaBlendedCheckerboard(!GetDrawAlphaBlendedCheckerboard());
						}),
						FCanExecuteAction::CreateLambda([this]
						{
							return IsDrawAlphaBlendedCheckerboardEnabled();
						}),
						FGetActionCheckState::CreateLambda([this]
						{
							return GetDrawAlphaBlendedCheckerboard() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					),
					EUserInterfaceActionType::ToggleButton
				);
				AlphaCheckerboardEntry.Visibility = TAttribute<bool>::CreateSP(this, &SMediaProfileMediaItemDisplayBase::CanShowAlphaCheckerboard);
			}),
			/*bInOpenSubMenuOnClick=*/false,
			TAttribute<FSlateIcon>::CreateLambda(
				[this]
				{
					const int8 ChannelMaskIndex = (int8)GetChannelMask();
					return FSlateIcon(FAppStyle::GetAppStyleSetName(), ChannelIcons[ChannelMaskIndex + 1]);
				})
		);

		return Entry;
	}

	FToolMenuEntry CreateChannelsButton()
	{
		return FToolMenuEntry::InitDynamicEntry(
			"Channels",
			FNewToolMenuSectionDelegate::CreateLambda(
				[](FToolMenuSection& InDynamicSection) -> void
				{
					if (UMediaProfileMediaItemDisplayContext* Context = InDynamicSection.FindContext<UMediaProfileMediaItemDisplayContext>())
					{
						TSharedPtr<SMediaProfileMediaItemDisplayBase<TMediaItem>> ContextWidget = Context->GetDisplayWidget<SMediaProfileMediaItemDisplayBase<TMediaItem>>();
						
						constexpr bool bUseFullLabel = false;
						FToolMenuEntry& Entry = InDynamicSection.AddEntry(ContextWidget->CreateChannelsSubmenu(false));
						Entry.Visibility = TAttribute<bool>::CreateLambda([]
						{
							return GetDefault<UMediaProfileEditorUserSettings>()->bShowChannelsInViewportToolbar;
						});
						Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
					}
				}
			)
		);
	}

	/** Creates the viewport arrangement menu, which allows users to change the arrangement of panels in the viewport */
	FToolMenuEntry CreateViewportArrangementMenu()
	{
		FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
			"ViewportArrangementMenu",
			LOCTEXT("ViewportDropdownLabel", "..."),
			LOCTEXT("ViewportArrangementTooltip", "Viewport arrangements"),
			FNewToolMenuDelegate::CreateLambda([this](UToolMenu* Submenu)
			{
				TSharedPtr<SMediaProfileViewport> PinnedOwningViewport = OwningViewport.Pin();
				if (!PinnedOwningViewport)
				{
					return;
				}
				
				auto AddLayoutSectionFn = [Submenu, CommandList = PinnedOwningViewport->GetCommandList()](const FName& SectionName, const FText& SectionLabel, const TArray<TSharedPtr<FUICommandInfo>>& Commands)
				{
					FToolMenuSection& Section = Submenu->AddSection(SectionName, SectionLabel);

					FSlimHorizontalToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None);
					ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
					ToolbarBuilder.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

					for (const TSharedPtr<FUICommandInfo>& Command : Commands)
					{
						ToolbarBuilder.AddToolBarButton(Command);
					}

					TSharedRef<SWidget> Widget = SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							ToolbarBuilder.MakeWidget()
						]
						
						+SHorizontalBox::Slot()
						.FillWidth(1)
						[
							SNullWidget::NullWidget
						];
					
					const FText Label = FText::GetEmpty();
					constexpr bool bNoIndent = true;
					Section.AddEntry(FToolMenuEntry::InitWidget(SectionName, Widget, Label, bNoIndent));
				};

				FLevelViewportCommands& Commands = FLevelViewportCommands::Get();
				
				AddLayoutSectionFn(TEXT("ViewportOnePaneConfigs"), LOCTEXT("OnePaneConfigHeader", "One Pane"), { Commands.ViewportConfig_OnePane });
				AddLayoutSectionFn(TEXT("ViewportTwoPaneConfigs"), LOCTEXT("TwoPaneConfigHeader", "Two Panes"), { Commands.ViewportConfig_TwoPanesH, Commands.ViewportConfig_TwoPanesV });
				AddLayoutSectionFn(TEXT("ViewportThreePaneConfigs"), LOCTEXT("ThreePaneConfigHeader", "Three Panes"), {
					Commands.ViewportConfig_ThreePanesLeft,
					Commands.ViewportConfig_ThreePanesRight,
					Commands.ViewportConfig_ThreePanesTop,
					Commands.ViewportConfig_ThreePanesBottom});
				AddLayoutSectionFn(TEXT("ViewportFourPaneConfigs"), LOCTEXT("FourPaneConfigHeader", "Four Panes"), {
					Commands.ViewportConfig_FourPanes2x2,
					Commands.ViewportConfig_FourPanesLeft,
					Commands.ViewportConfig_FourPanesRight,
					Commands.ViewportConfig_FourPanesTop,
					Commands.ViewportConfig_FourPanesBottom});

				
				FToolMenuSection& MaximizeSection = Submenu->FindOrAddSection("MaximizeSection");
			
				MaximizeSection.AddSeparator("MaximizeSeparator");
				MaximizeSection.AddEntry(FToolMenuEntry::InitMenuEntry(FLevelViewportCommands::Get().ToggleImmersive));
				MaximizeSection.AddEntry(CreateMaximizeViewportButton());
			}));

		Entry.StyleNameOverride = FName("ViewportToolbarViewportSizingSubmenu");
		Entry.InsertPosition.Position = EToolMenuInsertType::Last;
		Entry.ToolBarData.LabelOverride = FText();
		Entry.ToolBarData.ResizeParams.AllowClipping = false;

		return Entry;
	}

	/** Creates the menu entry to maximize this panel */
	FToolMenuEntry CreateMaximizeViewportButton()
	{
		FToolMenuEntry MaximizeRestoreEntry = FToolMenuEntry::InitMenuEntry(FLevelViewportCommands::Get().ToggleMaximize,
			TAttribute<FText>::CreateLambda([this]()
			{
				return IsMaximized() ?
					LOCTEXT("MaximizeRestoreLabel_Restore", "Restore All Viewports") :
					LOCTEXT("MaximizeRestoreLabel_Maximize", "Maximize Viewport");
			}),
			TAttribute<FText>::CreateLambda([this]()
			{
				return IsMaximized() ?
					LOCTEXT("MaximizeRestoreTooltip_Restore", "Restores the layout to show all viewports") :
					LOCTEXT("MaximizeRestoreTooltip_Maximize", "Maximizes this viewport");
			}),
			TAttribute<FSlateIcon>::CreateLambda([this]()
			{
				return IsMaximized() ?
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewportToolBar.Maximize.Checked") :
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewportToolBar.Maximize.Normal");
			})
		);
				
		MaximizeRestoreEntry.SetShowInToolbarTopLevel(true);
		MaximizeRestoreEntry.ToolBarData.ResizeParams.AllowClipping = false;
		MaximizeRestoreEntry.StyleNameOverride = FName("ViewportToolbarViewportSizingSubmenu");

		return MaximizeRestoreEntry;
	}

	FToolMenuEntry CreateSettingsMenu()
	{
		FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
			"SettingsMenu",
			TAttribute<FText>(),
			TAttribute<FText>(),
			FNewToolMenuDelegate::CreateLambda([this](UToolMenu* Submenu)
			{
				FToolMenuSection& TimecodeGenlockSection = Submenu->AddSection("TimecodeGenlockSettingsSection", LOCTEXT("TimecodeGenlockSettingsSectionLabel", "Timecode & Genlock"));
				TimecodeGenlockSection.AddMenuEntry(
					"ShowTimecodeInToolbar",
					LOCTEXT("ShowTimecodeInToolbarLabel", "Show Timecode in Viewport Toolbar"),
					LOCTEXT("ShowTimecodeInToolbarTooltip", "Whether to show the timecode in the viewport toolbar"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([]
						{
							if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
							{
								UserSettings->bShowTimecodeInViewportToolbar = !UserSettings->bShowTimecodeInViewportToolbar;
								UserSettings->SaveConfig();
							}
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([]
						{
							if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
							{
								return UserSettings->bShowTimecodeInViewportToolbar;
							}

							return true;
						})),
						EUserInterfaceActionType::ToggleButton);
				
				TimecodeGenlockSection.AddMenuEntry(
					"ShowGenlockInToolbar",
					LOCTEXT("ShowGenlockInToolbarLabel", "Show Genlock in Viewport Toolbar"),
					LOCTEXT("ShowGenlockInToolbarTooltip", "Whether to show the genlock status in the viewport toolbar"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([]
						{
							if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
							{
								UserSettings->bShowGenlockInViewportToolbar = !UserSettings->bShowGenlockInViewportToolbar;
								UserSettings->SaveConfig();
							}
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([]
						{
							if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
							{
								return UserSettings->bShowGenlockInViewportToolbar;
							}

							return true;
						})),
						EUserInterfaceActionType::ToggleButton);
			}),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings")
		);

		return Entry;
	}
	
	/** Gets the label to display for this panel's content in the media items combo button */
	virtual FText GetActiveMediaItemLabel() const
	{
		if (GetMediaItem())
		{
			return FText::Format(LOCTEXT("ActiveMediaItemLabelFormat", "{0} {1}: {2}"), GetBaseMediaTypeLabel(), MediaItemIndex + 1, GetMediaItemLabel());
		}

		return FText::Format(LOCTEXT("ActiveMediaItemLabelFormat", "{0} {1}: {2}"), GetBaseMediaTypeLabel(), MediaItemIndex + 1, LOCTEXT("MediaNotConfiguredLabel", "Media not configured"));
	}

	/** Gets the icon to display for this panel's content in the media items combo button */
	virtual FSlateIcon GetActiveMediaItemIcon() const
	{
		if (TMediaItem* MediaItem = GetMediaItem())
		{
			return FSlateIconFinder::FindIconForClass(MediaItem->GetClass());
		}

		return FSlateIconFinder::FindIconForClass(TMediaItem::StaticClass());
	}

	/** Raised when the user has selected a new media item in the media items combo button */
	virtual void ChangeActiveMediaItem(UClass* InMediaItemClass, int32 InMediaItemIndex)
	{
		bool bRefreshDisplay = true;
		if (InMediaItemClass->IsChildOf(TMediaItem::StaticClass()))
		{
			// If the media item is the same type as this widget's type, we can update the index and refresh the widget
			MediaItemIndex = InMediaItemIndex;
			ConfigureMediaImage();
			bRefreshDisplay = false;
		}

		ChangePanelContents(InMediaItemClass, InMediaItemIndex, bRefreshDisplay);
	}

	/** Gets whether 2D visual transformations are allowed on the media item */
	virtual bool CanTransformMediaItem() const
	{
		return false;
	}
	
	/** Gets the media item's rotation enum value */
	virtual EMediaImageRotation GetRotation()
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return MediaItemSettings->Rotation;
		}
		
		return EMediaImageRotation::None;
	}
	
	/** Gets the media item's actual rotation value */
	virtual float GetRotationValue(bool bInDegrees = false) const
	{
		float RotationInDegrees = 0.0f;
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			switch (MediaItemSettings->Rotation)
			{
			case EMediaImageRotation::CW90:
				RotationInDegrees = -90.0f;
				break;
			
			case EMediaImageRotation::CCW90:
				RotationInDegrees = 90.0f;
				break;
			
			case EMediaImageRotation::CW180:
				RotationInDegrees = 180.0f;
				break;
			
			default:
				RotationInDegrees = 0.0f;
				break;
			}
		}
	
		return bInDegrees ? RotationInDegrees : FMath::DegreesToRadians(RotationInDegrees);
	}
	
	/** Gets the check state for the specified rotation enum value for the media item */
	virtual ECheckBoxState GetRotationCheckState(EMediaImageRotation InRotation)
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return MediaItemSettings->Rotation == InRotation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		
		return ECheckBoxState::Unchecked;
	}
	
	/** Sets the media item's rotation */
	virtual void SetRotation(EMediaImageRotation InRotation)
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			MediaItemSettings->Rotation = InRotation;
		}
	}
	
	/** Gets the flip state of the media item */
	virtual void GetFlipState(bool& bOutFlipHorizontal, bool& bOutFlipVertical)
	{
		bOutFlipHorizontal = false;
		bOutFlipVertical = false;
		
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			bOutFlipHorizontal = MediaItemSettings->bFlipHorizontal;
			bOutFlipVertical = MediaItemSettings->bFlipVertical;
		}
	}
	
	/** Gets the check state for the media item's horizontal flip state */
	virtual ECheckBoxState GetFlipHorizontalCheckState()
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return MediaItemSettings->bFlipHorizontal ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		
		return ECheckBoxState::Unchecked;
	}
	
	/** Gets the check state for the media item's vertical flip state */
	virtual ECheckBoxState GetFlipVerticalCheckState()
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return MediaItemSettings->bFlipVertical ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		
		return ECheckBoxState::Unchecked;
	}
	
	/** Toggle's the media item's horizontal flip state */
	virtual void ToggleFlipHorizontal()
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			MediaItemSettings->bFlipHorizontal = !MediaItemSettings->bFlipHorizontal;
		}
	}
	
	/** Toggle's the media item's vertical flip state */
	virtual void ToggleFlipVertical()
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			MediaItemSettings->bFlipVertical = !MediaItemSettings->bFlipVertical;
		}
	}
	
	/** Gets the scale transform that should be applied to the media item's visualization */
	virtual FVector2f GetScale() const
	{
		FVector2f Scale = FVector2f::UnitVector;
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			if (MediaItemSettings->bFlipHorizontal)
			{
				Scale.X *= -1.0f;
			}
			
			if (MediaItemSettings->bFlipVertical)
			{
				Scale.Y *= -1.0f;
			}
		}
		
		return Scale;
	}
	
	/** Resets any active transform on the media item */
	virtual void ResetTransform()
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			MediaItemSettings->Rotation = EMediaImageRotation::None;
			MediaItemSettings->bFlipHorizontal = false;
			MediaItemSettings->bFlipVertical = false;
		}
	}

	/** Gets whether the media item has a transformation that can be reset */
	virtual bool CanResetTransform() const
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return MediaItemSettings->Rotation != EMediaImageRotation::None ||
				MediaItemSettings->bFlipHorizontal ||
				MediaItemSettings->bFlipVertical;
		}
		
		return false;
	}

	/** Gets the current color channel being displayed in the viewport */
	virtual EColorChannelMask GetChannelMask() const
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return (EColorChannelMask)MediaItemSettings->ColorChannelMask;
		}

		return EColorChannelMask::All;
	}

	/** Sets the current color channel being displayed in the viewport */
	virtual void SetChannelMask(EColorChannelMask InChannelMask)
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			MediaItemSettings->ColorChannelMask = (int32)InChannelMask;
		}
	}

	/** Gets whether the specified color channel is the one being masked to */
	virtual bool IsChannelBeingMasked(EColorChannelMask InChannelMask)
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return (EColorChannelMask)MediaItemSettings->ColorChannelMask == InChannelMask;
		}

		return false;
	}
	
	/** Gets whether the alpha color channel is inverted when displayed as the color channel mask */
	virtual bool GetInvertAlphaChannelMask() const
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return MediaItemSettings->bInvertAlphaChannelMask;
		}

		return false;
	}

	/** Gets whether the alpha color channel is inverted when displayed as the color channel mask */
	virtual void SetInvertAlphaChannelMask(bool bInInvertAlphaChannelMask)
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			MediaItemSettings->bInvertAlphaChannelMask = bInInvertAlphaChannelMask;
		}
	}

	/** Gets whether the display's alpha channel can be displayed as inverted or not */
	virtual bool CanInvertAlphaChannelMask() const
	{
		return false;
	}
	
	/** Gets whether to draw the alpha blended checkerboard background when rendering the viewport */
	virtual bool GetDrawAlphaBlendedCheckerboard() const
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			return MediaItemSettings->bDrawAlphaBlendedCheckerboard;
		}

		return false;
	}

	/** Sets whether to draw the alpha blended checkerboard background when rendering the viewport */
	virtual void SetDrawAlphaBlendedCheckerboard(bool bInDrawAlphaBlendedCheckerboard)
	{
		if (FMediaProfileEditorPerMediaItemSettings* MediaItemSettings = GetMediaItemSettings())
		{
			MediaItemSettings->bDrawAlphaBlendedCheckerboard = bInDrawAlphaBlendedCheckerboard;
		}
	}

	virtual bool IsDrawAlphaBlendedCheckerboardEnabled() const
	{
		return true;
	}

	/** Indicates whether the media item display can show the 'Visualize Alpha with Checkerboard' option in the toolbar dropdown menu */
	virtual bool CanShowAlphaCheckerboard() const
	{
		return true;	
	}
	
	/** Gets whether this panel is currently maximized in the viewport */
	bool IsMaximized() const
	{
		TSharedPtr<SMediaProfileViewport> PinnedOwningViewport = OwningViewport.Pin();
		if (!PinnedOwningViewport.IsValid())
		{
			return false;
		}

		return PinnedOwningViewport->IsPanelMaximized(PanelIndex);
	}

	/** Gets whether this panel is currently immersive in the viewport */
	bool IsImmersive() const
	{
		TSharedPtr<SMediaProfileViewport> PinnedOwningViewport = OwningViewport.Pin();
		if (!PinnedOwningViewport.IsValid())
		{
			return false;
		}

		return PinnedOwningViewport->IsPanelImmersive();
	}

	/** Changes the panel's contents, recreating the display widgets */
	void ChangePanelContents(UClass* InMediaItemClass, int32 InMediaItemIndex, bool bRefreshDisplay)
	{
		TSharedPtr<SMediaProfileViewport> PinnedOwningViewport = OwningViewport.Pin();
		if (!PinnedOwningViewport.IsValid())
		{
			return;
		}

		PinnedOwningViewport->SetPanelContents(PanelIndex, InMediaItemClass, InMediaItemIndex, bRefreshDisplay);
	}

	/** Raised when the lock button is toggled */
	void OnPanelLockToggled()
	{
		TSharedPtr<SMediaProfileViewport> PinnedOwningViewport = OwningViewport.Pin();
		if (!PinnedOwningViewport.IsValid())
		{
			return;
		}

		PinnedOwningViewport->SetPanelLocked(PanelIndex, !IsPanelLocked());
	}

	/** Gets whether this panel is locked or not */
	bool IsPanelLocked() const
	{
		return OwningViewport.IsValid() && OwningViewport.Pin()->IsPanelLocked(PanelIndex);
	}

	/** Gets the lock button icon */
	FSlateIcon GetLockIcon() const
	{
		return IsPanelLocked() ?
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.Locked")) :
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.Unlocked"));
	}

	FMediaProfileEditorPerMediaItemSettings* GetMediaItemSettings() const
	{
		if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
		{
			if (TMediaItem* MediaItem = GetMediaItem())
			{
				FName MediaItemName = MediaItem->GetFName();
				if (!UserSettings->PerMediaItemSettings.Contains(MediaItemName))
				{
					UserSettings->PerMediaItemSettings.Add(MediaItemName, FMediaProfileEditorPerMediaItemSettings());
				}

				return &UserSettings->PerMediaItemSettings[MediaItemName];
			}
		}

		return nullptr;
	}
	
	/** Raised when an object property is changed in the editor, used to potentially refresh the display if the displayed media item is modified */
	virtual void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
	{
		if (!InObject)
		{
			return;
		}
		
		if (InObject == GetMediaItem())
		{
			ConfigureMediaImage();
		}
	}
	
protected:
	/** The media profile editor that owns this display */
	TWeakPtr<FMediaProfileEditor> MediaProfileEditor;

	/** The viewport widget that owns this display */
	TWeakPtr<SMediaProfileViewport> OwningViewport;

	/** The index of the panel that is displaying this display */
	int32 PanelIndex = INDEX_NONE;

	/** The index of the media item being displayed by this display */
	int32 MediaItemIndex = INDEX_NONE;

	/** Entry for the media profile's timecode and genlock configuration to display in the viewport toolbar */
	TSharedPtr<FMediaFrameworkTimecodeGenlockToolMenuEntry> TimecodeGenlockToolMenuEntry;
	
	TSharedPtr<SOverlay> Overlay;
	TSharedPtr<SBorder> MediaImageContainer;
	TSharedPtr<FUICommandList> CommandList;
};

#undef LOCTEXT_NAMESPACE

/** Implementation of SMediaProfileMediaItemDisplayBase for media sources, which displays the live feed from the selected media source */
class SMediaProfileMediaSourceDisplay : public SMediaProfileMediaItemDisplayBase<UMediaSource>
{
public:
	SLATE_BEGIN_ARGS(SMediaProfileMediaSourceDisplay) {}
		SLATE_ARGUMENT(TWeakPtr<FMediaProfileEditor>, MediaProfileEditor)
		SLATE_ARGUMENT(int32, PanelIndex)
		SLATE_ARGUMENT(int32, MediaItemIndex)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<SMediaProfileViewport>& InOwningViewport);

	virtual ~SMediaProfileMediaSourceDisplay() override;

	//~ Begin SWidget
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

protected:
	virtual void ConfigureMediaImage() override;

	virtual UMediaSource* GetMediaItem() const override;
	virtual FText GetMediaItemLabel() const override;
	virtual FText GetBaseMediaTypeLabel() const override;
	virtual bool CanTransformMediaItem() const override;

private:
	UMediaTexture* GetMediaTexture() const;
	FVector2D GetSourceImageSize() const;
	FOptionalSize GetSourceAspectRatio() const;

	FText GetTimeDurationText() const;
	FText GetFramerateText() const;

	/** Register a tile-visibility provider for InMediaTexture. Must be called before opening the media source. */
	void SetupTileVisibilityProvider(UMediaTexture* InMediaTexture);

	/** Symmetric counterpart to SetupTileVisibilityProvider. Idempotent; safe to call when nothing is registered. */
	void TeardownTileVisibilityProvider();

	/** The actual masked-image leaf widget; Tick reads its tick-space geometry to push the rendered size to the provider. */
	TSharedPtr<SMaskedMediaImage> MediaImage;

	/** Tile-visibility provider; Tick pushes the rendered viewport size into it each frame. */
	TSharedPtr<UE::MediaPlayerEditor::FViewportTileVisibilityProvider, ESPMode::ThreadSafe> TileVisibilityProvider;

	/** FMediaTextureTracker bookkeeping for the provider. Registered against the playback manager's MediaTexture. */
	TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe> TextureTrackerObject;

	/** The MediaTexture we registered with on the tracker - kept so the dtor can unregister against the same one. */
	TWeakObjectPtr<UMediaTexture> RegisteredMediaTexture;
};

/** Implementation of SMediaProfileMediaItemDisplayBase for media outputs, which displays the result from a media capture from the selected media output */
class SMediaProfileMediaOutputDisplay : public SMediaProfileMediaItemDisplayBase<UMediaOutput>
{
private:
	enum class ECaptureType : uint8
	{
		None,
		ActiveViewport,
		Viewport,
		RenderTarget,
		MediaTexture
	};
	
public:
	SLATE_BEGIN_ARGS(SMediaProfileMediaOutputDisplay) {}
		SLATE_ARGUMENT(TWeakPtr<FMediaProfileEditor>, MediaProfileEditor)
		SLATE_ARGUMENT(int32, PanelIndex)
		SLATE_ARGUMENT(int32, MediaItemIndex)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<SMediaProfileViewport>& InOwningViewport);

	virtual ~SMediaProfileMediaOutputDisplay() override;
	
protected:
	virtual void ConfigureMediaImage() override;
	
	virtual UMediaOutput* GetMediaItem() const override;
	virtual FText GetMediaItemLabel() const override;
	virtual FText GetBaseMediaTypeLabel() const override;

	virtual bool CanTransformMediaItem() const override;
	virtual EMediaImageRotation GetRotation() override;
	virtual ECheckBoxState GetRotationCheckState(EMediaImageRotation InRotation) override;
	virtual float GetRotationValue(bool bInDegrees = false) const override;
	virtual void SetRotation(EMediaImageRotation InRotation) override;
	virtual void GetFlipState(bool& bOutFlipHorizontal, bool& bOutFlipVertical) override;
	virtual ECheckBoxState GetFlipHorizontalCheckState() override;
	virtual ECheckBoxState GetFlipVerticalCheckState() override;
	virtual void ToggleFlipHorizontal() override;
	virtual void ToggleFlipVertical() override;
	virtual FVector2f GetScale() const override;
	virtual void ResetTransform() override;
	virtual bool CanResetTransform() const override;

	virtual void SetChannelMask(EColorChannelMask InChannelMask) override;
	virtual void SetInvertAlphaChannelMask(bool bInInvertAlphaChannelMask) override;
	virtual bool CanInvertAlphaChannelMask() const override { return true; }
	virtual void SetDrawAlphaBlendedCheckerboard(bool bInDrawAlphaBlendedCheckerboard) override;
	virtual bool IsDrawAlphaBlendedCheckerboardEnabled() const override;
	virtual bool CanShowAlphaCheckerboard() const override;

	virtual void AddToolbarEntries(FToolMenuSection& Section) override;
	virtual void AppendToolbarCommandList(FToolMenuContext& Context) override;
	
	virtual void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent) override;

private:
	TOptional<EMediaCaptureState> GetMediaOutputCaptureState() const;
	FOptionalSize GetMediaOutputDesiredAspectRatio() const;
	
	void OnCaptureMethodChanged(UMediaOutput* MediaOutput);
	
private:
	TSharedPtr<SWidget> ImageWidget;
	TSharedPtr<FUICommandList> ShowFlagsCommandList;
	ECaptureType CaptureType = ECaptureType::None;
	int32 CaptureIndex = INDEX_NONE;
	bool bDisplayShowFlags = false;
};

/** Dummy display for when a panel needs to be displayed that doesn't contain any media item */
class SMediaProfileDummyDisplay : public SMediaProfileMediaItemDisplayBase<UObject>
{
public:
	SLATE_BEGIN_ARGS(SMediaProfileDummyDisplay) {}
		SLATE_ARGUMENT(TWeakPtr<FMediaProfileEditor>, MediaProfileEditor)
		SLATE_ARGUMENT(int32, PanelIndex)
	SLATE_END_ARGS()
		
	void Construct(const FArguments& InArgs, const TSharedPtr<SMediaProfileViewport>& InOwningViewport);

protected:
	virtual void ConfigureMediaImage() override;
	
	virtual UObject* GetMediaItem() const override { return nullptr; }
	virtual FText GetMediaItemLabel() const override { return FText::GetEmpty(); }
	virtual FText GetBaseMediaTypeLabel() const override { return FText::GetEmpty(); }

	virtual FText GetActiveMediaItemLabel() const override;
	virtual FSlateIcon GetActiveMediaItemIcon() const override { return FSlateIcon(); }
	virtual void ChangeActiveMediaItem(UClass* InMediaItemClass, int32 InMediaItemIndex) override;
};