// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaProfileSourcesTreeView.h"

#include "Editor.h"
#include "MediaOutput.h"
#include "MediaProfileEditor.h"
#include "MediaProfileEditorUserSettings.h"
#include "MediaSource.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "SPositiveActionButton.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "UnrealExporter.h"
#include "CaptureTab/SMediaFrameworkCapture.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Exporters/Exporter.h"
#include "Filters/GenericFilter.h"
#include "Filters/SBasicFilterBar.h"
#include "Filters/SFilterBar.h"
#include "Filters/SFilterSearchBox.h"
#include "Framework/Commands/GenericCommands.h"
#include "MediaAssets/ProxyMediaOutput.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "Misc/MessageDialog.h"
#include "Misc/StringOutputDevice.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfileSettings.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "MediaProfileSourcesTreeView"

const FLazyName SMediaProfileSourcesTreeView::Column_Index(TEXT("Column_Index"));
const FLazyName SMediaProfileSourcesTreeView::Column_ItemLabel(TEXT("Column_ItemLabel"));
const FLazyName SMediaProfileSourcesTreeView::Column_ItemType(TEXT("Column_ItemType"));
const FLazyName SMediaProfileSourcesTreeView::Column_Configuration(TEXT("Column_Configuration"));

namespace MediaSourcesTreeView
{
	/** Attempts to saves the media profile settings to the project's config files */
	void TrySaveMediaProfileSettings()
	{
		// This will log a warning if the default file was read-only
		const bool bUpdated = GetMutableDefault<UMediaProfileSettings>()->TryUpdateDefaultConfigFile();

		// If the default settings could not be updated, try once to make the default config file writable, then try updating it again
		if (!bUpdated)
		{
			const FString RelativeConfigFilePath = GetMutableDefault<UMediaProfileSettings>()->GetDefaultConfigFilename();
			const FString TargetFilePath = FPaths::ConvertRelativePathToFull(RelativeConfigFilePath);

			constexpr bool bForceSourceControlUpdate = false;
			constexpr bool bShowErrorInNotification = true;
			if (SettingsHelpers::CheckOutOrAddFile(TargetFilePath, bForceSourceControlUpdate, bShowErrorInNotification))
			{
				GetMutableDefault<UMediaProfileSettings>()->TryUpdateDefaultConfigFile();
			}
		}
	}
}

/** Slate widget for the individual rows of the media tree view */
class SMediaTreeItemRow : public SMultiColumnTableRow<SMediaProfileSourcesTreeView::FMediaTreeItemPtr>
{
private:
	/** Drag/drop operation for changing the order of sources or outputs in the tree view */
	class FMediaDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FMediaDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FMediaDragDropOp> New(const SMediaProfileSourcesTreeView::FMediaTreeItemPtr& InDraggedItem)
		{
			TSharedRef<FMediaDragDropOp> Operation = MakeShared<FMediaDragDropOp>();
			Operation->DraggedItem = InDraggedItem;
			Operation->Construct();
			return Operation;
		}

		virtual void Construct() override
		{
			if (DraggedItem.IsValid())
			{
				SetToolTip(DraggedItem.Pin()->Label, DraggedItem.Pin()->Icon);
			}
			
			FDecoratedDragDropOp::Construct();
		}

		TWeakPtr<SMediaProfileSourcesTreeView::FMediaTreeItem> DraggedItem;
	};
	
public:
	/** Delegate that is called when a media item is moved. Parameters are the item being moved, and the index to move it to  */
	DECLARE_DELEGATE_TwoParams(FOnMoveMediaItem, SMediaProfileSourcesTreeView::FMediaTreeItemPtr, int32)
	
	SLATE_BEGIN_ARGS(SMediaTreeItemRow) {}
		SLATE_EVENT(FOnMoveMediaItem, OnMoveMediaItem)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, SMediaProfileSourcesTreeView::FMediaTreeItemPtr InTreeItem)
	{
		TreeItem = InTreeItem;
		OnMoveMediaItem = InArgs._OnMoveMediaItem;
		
		STableRow<SMediaProfileSourcesTreeView::FMediaTreeItemPtr>::FArguments Args = FSuperRowType::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
			.OnDragDetected(this, &SMediaTreeItemRow::HandleDragDetected)
			.OnCanAcceptDrop(this, &SMediaTreeItemRow::HandleCanAcceptDrop)
			.OnAcceptDrop(this, &SMediaTreeItemRow::HandleAcceptDrop);
		
		SMultiColumnTableRow<SMediaProfileSourcesTreeView::FMediaTreeItemPtr>::Construct(Args, InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		SMediaProfileSourcesTreeView::FMediaTreeItemPtr PinnedTreeItem = TreeItem.Pin();
		if (!PinnedTreeItem.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		auto CreateColumnWidget = [](const FText& InLabel, const FSlateBrush* InIcon)
		{
			TSharedPtr<SWidget> LabelWidget = SNullWidget::NullWidget;
			if (!InLabel.IsEmptyOrWhitespace())
			{
				LabelWidget = SNew(STextBlock).Text(InLabel);
			}

			TSharedPtr<SWidget> IconWidget = SNullWidget::NullWidget;
			if (InIcon)
			{
				IconWidget = SNew(SBox)
					.WidthOverride(16.0f)
					.HeightOverride(16.0f)
					[
						SNew(SImage).Image(InIcon)
					];
			}

			const FMargin LeftMargin = InIcon ? FMargin(6.0f, 1.0f, 6.0f, 1.0f) : FMargin();
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(LeftMargin)
				.AutoWidth()
				[
					IconWidget.ToSharedRef()
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					LabelWidget.ToSharedRef()
				];
		};
		
		if (ColumnName == SMediaProfileSourcesTreeView::Column_Index)
		{
			if (PinnedTreeItem->IsLeaf())
			{
				return CreateColumnWidget(FText::Format(LOCTEXT("MediaSourceIndexLabelFormat", "{0}"), PinnedTreeItem->Index + 1), nullptr);
			}
		}

		if (ColumnName == SMediaProfileSourcesTreeView::Column_ItemLabel)
		{
			return SNew(SBox)
				.MinDesiredHeight(20.0f)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6.f, 0.f, 0.f, 0.f)
					[
						SNew(SExpanderArrow, SharedThis(this)).IndentAmount(12)
					]

					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						CreateColumnWidget(PinnedTreeItem->Label, PinnedTreeItem->Icon)
					]
				];
		}

		if (ColumnName == SMediaProfileSourcesTreeView::Column_ItemType)
		{
			if (PinnedTreeItem->IsLeaf())
			{
				return CreateColumnWidget(PinnedTreeItem->Type, PinnedTreeItem->Icon);
			}
		}

		if (ColumnName == SMediaProfileSourcesTreeView::Column_Configuration)
		{
			if (PinnedTreeItem->IsLeaf())
			{
				return CreateColumnWidget(PinnedTreeItem->Configuration, nullptr);
			}
		}
		
		return SNullWidget::NullWidget;
	}

private:
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (!TreeItem.IsValid())
		{
			return FReply::Unhandled();
		}

		if (!TreeItem.Pin()->IsLeaf())
		{
			return FReply::Unhandled();
		}
		
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			TSharedPtr<FDragDropOperation> DragDropOp = FMediaDragDropOp::New(TreeItem.Pin());
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}

		return FReply::Unhandled();
	}

	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, SMediaProfileSourcesTreeView::FMediaTreeItemPtr InTreeItem)
	{
		TSharedPtr<FMediaDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FMediaDragDropOp>();

		if (!DragDropOp.IsValid() || !DragDropOp->DraggedItem.IsValid())
		{
			return TOptional<EItemDropZone>();
		}

		SMediaProfileSourcesTreeView::FMediaTreeItemPtr DraggedItem = DragDropOp->DraggedItem.Pin();
		if (DraggedItem->BackingClass == InTreeItem->BackingClass)
		{
			if (!InTreeItem->IsLeaf())
			{
				return EItemDropZone::BelowItem;
			}

			return InItemDropZone == EItemDropZone::OntoItem ? EItemDropZone::AboveItem : InItemDropZone;
		}
		
		return TOptional<EItemDropZone>();
	}

	
	FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, SMediaProfileSourcesTreeView::FMediaTreeItemPtr InTreeItem)
	{
		TSharedPtr<FMediaDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FMediaDragDropOp>();

		if (!DragDropOp.IsValid() || !DragDropOp->DraggedItem.IsValid())
		{
			return FReply::Unhandled();
		}

		if (OnMoveMediaItem.IsBound())
		{
			int32 DestIndex = InItemDropZone == EItemDropZone::AboveItem ? InTreeItem->Index: InTreeItem->Index + 1;

			// If the destination is further down than the item being moved, we must subtract one from the destination index
			// to account for the removal of the original item
			if (InTreeItem->Index > DragDropOp->DraggedItem.Pin()->Index)
			{
				--DestIndex;
			}
			
			OnMoveMediaItem.Execute(DragDropOp->DraggedItem.Pin(), DestIndex);
			return FReply::Handled();
		}
		
		return FReply::Unhandled();
	}
	
private:
	TWeakPtr<SMediaProfileSourcesTreeView::FMediaTreeItem> TreeItem;

	FOnMoveMediaItem OnMoveMediaItem;
};

/**
 * Widget for the filter bar, needs to be a subclass that overrides MakeAddFilterMenu to give the filter bar its own unique menu name,
 * otherwise the editor will get confused with any other basic filter bar used elsewhere.
 */
template<typename TFilterType>
class SMediaSourcesFilterBar : public SBasicFilterBar<TFilterType>
{
	using Super = SBasicFilterBar<TFilterType>;
	
public:

	using FOnFilterChanged = typename SBasicFilterBar<TFilterType>::FOnFilterChanged;
	using FCreateTextFilter = typename SBasicFilterBar<TFilterType>::FCreateTextFilter;

	SLATE_BEGIN_ARGS(SMediaSourcesFilterBar)
	{}
		SLATE_EVENT(SMediaSourcesFilterBar<TFilterType>::FOnFilterChanged, OnFilterChanged)
		SLATE_ARGUMENT(TArray<TSharedRef<FFilterBase<TFilterType>>>, CustomFilters)	
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		typename SBasicFilterBar<TFilterType>::FArguments Args;
		Args._OnFilterChanged = InArgs._OnFilterChanged;
		Args._CustomFilters = InArgs._CustomFilters;
		Args._UseSectionsForCategories = true;
		
		SBasicFilterBar<TFilterType>::Construct(Args.FilterPillStyle(EFilterPillStyle::Basic));
	}

private:
	virtual TSharedRef<SWidget> MakeAddFilterMenu() override
	{
		const FName FilterMenuName = "MediaSourcesFilterBar.FilterMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
			Menu->bShouldCloseWindowAfterMenuSelection = true;
			Menu->bCloseSelfOnly = true;

			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				if (UFilterBarContext* Context = InMenu->FindContext<UFilterBarContext>())
				{
					Context->PopulateFilterMenu.ExecuteIfBound(InMenu);
					Context->OnExtendAddFilterMenu.ExecuteIfBound(InMenu);
				}
			}));
		}

		UFilterBarContext* FilterBarContext = NewObject<UFilterBarContext>();
		FilterBarContext->PopulateFilterMenu = FOnPopulateAddFilterMenu::CreateLambda([this](UToolMenu* Menu)
		{
			Super::PopulateCommonFilterSections(Menu);
			Super::PopulateCustomFilters(Menu);
		});
		FToolMenuContext ToolMenuContext(FilterBarContext);
		
		return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
	}
};

/** Toolbar widget that contains filtering, the add button, and the settings button */
class SMediaProfileSourcesToolbar : public SCompoundWidget
{
private:
	typedef SMediaProfileSourcesTreeView::FMediaTreeItemPtr FFilterType;
	typedef TTextFilter<FFilterType> FMediaTreeFilter;
	
public:
	DECLARE_DELEGATE(FOnFilterChanged)
	DECLARE_DELEGATE(FOnMediaSourceAdded)
	DECLARE_DELEGATE(FOnMediaOutputAdded)
	
	SLATE_BEGIN_ARGS(SMediaProfileSourcesToolbar) {}
		SLATE_EVENT(FOnFilterChanged, OnFilterChanged)
		SLATE_EVENT(FOnMediaSourceAdded, OnMediaSourceAdded)
		SLATE_EVENT(FOnMediaOutputAdded, OnMediaOutputAdded)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UMediaProfile* InMediaProfile)
	{
		MediaProfile = InMediaProfile;
		OnFilterChanged = InArgs._OnFilterChanged;
		OnMediaSourceAdded = InArgs._OnMediaSourceAdded;
		OnMediaOutputAdded = InArgs._OnMediaOutputAdded;

		InitializeFilters();
		
		FilterBar = SNew(SMediaSourcesFilterBar<FFilterType>)
			.CustomFilters(CustomFilters)
			.OnFilterChanged(this, &SMediaProfileSourcesToolbar::FilterChanged);
		
		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SPositiveActionButton)
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
					.Text(LOCTEXT("AddMediaButtonLabel", "Add"))
					.ToolTipText(LOCTEXT("AddMediaButtonToolTip", "Add new media source or media output"))
					.OnGetMenuContent(this, &SMediaProfileSourcesToolbar::GetAddMediaMenuContent)
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				.AutoWidth()
				[
					SMediaSourcesFilterBar<FFilterType>::MakeAddFilterButton(FilterBar.ToSharedRef())
				]
				
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SFilterSearchBox)
					.HintText(LOCTEXT("FilterSearch", "Search..."))
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search for specific media sources or outputs"))
					.OnTextChanged_Lambda([this](const FText& InText)
					{
						MediaTreeTextFilter->SetRawFilterText(InText);
					})
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
					//.OnGetMenuContent( this, &SSceneOutliner::GetViewButtonContent, Mode->ShowFilterOptions())
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				FilterBar.ToSharedRef()
			]
		];
	}

	bool ItemPassesFilters(const FFilterType& InItem) const
	{
		// Performs an OR check on any of the filters that are enabled in the filter bar, mirroring the scene outliner filter bar behavior
		auto PassesAnyFilterBarFilter = [this](const FFilterType& InItem)
		{
			TSharedPtr<TFilterCollection<FFilterType>> FilterCollection = FilterBar->GetAllActiveFilters();
			const int32 NumFilters = FilterCollection->Num();
			if (NumFilters == 0)
			{
				return true;
			}
		
			for (const TSharedPtr<IFilter<FFilterType>>& Filter : *FilterCollection.Get())
			{
				if (Filter->PassesFilter(InItem))
				{
					return true;
				}
			}

			return false;
		};
		
		return MediaTreeTextFilter->PassesFilter(InItem) && PassesAnyFilterBarFilter(InItem);
	}
	
private:
	/** Creates all the filter bar filters and initializes the text filter */
	void InitializeFilters()
	{
		// Basic filters, for filtering out the sources or outputs broadly
		{
			TSharedPtr<FFilterCategory> BasicFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("BasicFiltersCategory", "Basic"), FText::GetEmpty());

			TSharedPtr<FGenericFilter<FFilterType>> MediaSourcesFilter = MakeShared<FGenericFilter<FFilterType>>(
				BasicFiltersCategory,
				TEXT("MediaSourcesFilter"),
				LOCTEXT("MediaSourcesFilterName", "Sources"),
				FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([](FFilterType InItem)
				{
					if (!InItem.IsValid())
					{
						return false;
					}

					return InItem->BackingObject.IsValid() && InItem->BackingObject->IsA<UMediaSource>();
				}));
		
			MediaSourcesFilter->SetToolTipText(LOCTEXT("MediaSourcesFilter", "Only show media sources"));
			CustomFilters.Add(MediaSourcesFilter.ToSharedRef());

			TSharedPtr<FGenericFilter<FFilterType>> MediaOutputsFilter = MakeShared<FGenericFilter<FFilterType>>(
				BasicFiltersCategory,
				TEXT("MediaOutputsFilter"),
				LOCTEXT("MediaOutputsFilterName", "Outputs"),
				FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([](FFilterType InItem)
				{
					if (!InItem.IsValid())
					{
						return false;
					}

					return InItem->BackingObject.IsValid() && InItem->BackingObject->IsA<UMediaOutput>();
				}));
		
			MediaOutputsFilter->SetToolTipText(LOCTEXT("MediaOutputsFilter", "Only show media outputs"));
			CustomFilters.Add(MediaOutputsFilter.ToSharedRef());
		}

		// Class filters for media sources
		{
			TSharedPtr<FFilterCategory> SourceFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("SourceFiltersCategory", "Media Sources"), FText::GetEmpty());
			
			TArray<UClass*> ChildClasses;
			GetDerivedClasses(UMediaSource::StaticClass(), ChildClasses);

			for (UClass* ChildClass : ChildClasses)
			{
				if (ChildClass->HasAnyClassFlags(CLASS_Abstract))
				{
					continue;
				}
				
				TSharedPtr<FGenericFilter<FFilterType>> MediaSourceSubclassFilter = MakeShared<FGenericFilter<FFilterType>>(
				SourceFiltersCategory,
				FString::Format(TEXT("{0}ClassFilter"), { ChildClass->GetName() }),
				FText::Format(LOCTEXT("MediaSourceSubclassFilterName", "{0}"), ChildClass->GetDisplayNameText()),
				FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([ChildClass](FFilterType InItem)
				{
					if (!InItem.IsValid())
					{
						return false;
					}

					return InItem->BackingObject.IsValid() && InItem->BackingObject->IsA(ChildClass);
				}));

				CustomFilters.Add(MediaSourceSubclassFilter.ToSharedRef());
			}
		}

		// Class filters for media outputs
		{
			TSharedPtr<FFilterCategory> OutputFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("OutputFiltersCategory", "Media Outputs"), FText::GetEmpty());
			
			TArray<UClass*> ChildClasses;
			GetDerivedClasses(UMediaOutput::StaticClass(), ChildClasses);

			for (UClass* ChildClass : ChildClasses)
			{
				if (ChildClass->HasAnyClassFlags(CLASS_Abstract))
				{
					continue;
				}
				
				TSharedPtr<FGenericFilter<FFilterType>> MediaOutputSubclassFilter = MakeShared<FGenericFilter<FFilterType>>(
				OutputFiltersCategory,
				FString::Format(TEXT("{0}ClassFilter"), { ChildClass->GetName() }),
				FText::Format(LOCTEXT("MediaOutputSubclassFilterName", "{0}"), ChildClass->GetDisplayNameText()),
				FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([ChildClass](FFilterType InItem)
				{
					if (!InItem.IsValid())
					{
						return false;
					}

					return InItem->BackingObject.IsValid() && InItem->BackingObject->IsA(ChildClass);
				}));

				CustomFilters.Add(MediaOutputSubclassFilter.ToSharedRef());
			}
		}
		
		MediaTreeTextFilter = MakeShared<FMediaTreeFilter>(FMediaTreeFilter::FItemToStringArray::CreateLambda([](const FFilterType& InItem, OUT TArray<FString>& OutStrings)
		{
			if (InItem.IsValid())
			{
				OutStrings.Add(InItem->Label.ToString());
				OutStrings.Add(InItem->Type.ToString());
				OutStrings.Add(InItem->Configuration.ToString());
			}
		}));
		MediaTreeTextFilter->OnChanged().AddSP(this, &SMediaProfileSourcesToolbar::FilterChanged);
	}
	
	TSharedRef<SWidget> GetAddMediaMenuContent()
	{
		constexpr bool bCloseMenuAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddMediaSourceMenuLabel", "Add Media Source"),
			LOCTEXT("AddMediaSourceMenuToolTip", "Add a new media source to the profile"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() { OnMediaSourceAdded.ExecuteIfBound(); })));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddMediaOutputMenuLabel", "Add Media Output"),
			LOCTEXT("AddMediaOutputMenuToolTip", "Add a new media output to the profile"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() { OnMediaOutputAdded.ExecuteIfBound(); })));
		
		return MenuBuilder.MakeWidget();
	}

	void FilterChanged()
	{
		OnFilterChanged.ExecuteIfBound();
	}
	
private:
	TWeakObjectPtr<UMediaProfile> MediaProfile;
	TArray<TSharedRef<FFilterBase<FFilterType>>> CustomFilters;
	TSharedPtr<FMediaTreeFilter> MediaTreeTextFilter;
	
	TSharedPtr<SMediaSourcesFilterBar<FFilterType>> FilterBar;
	
	FOnFilterChanged OnFilterChanged;
	FOnMediaSourceAdded OnMediaSourceAdded;
	FOnMediaOutputAdded OnMediaOutputAdded;
};

void SMediaProfileSourcesTreeView::Construct(const FArguments& InArgs, UMediaProfile* InMediaProfile)
{
	MediaProfile = InMediaProfile;
	OnMediaItemDeleted = InArgs._OnMediaItemDeleted;
	OnSelectedMediaItemsChanged = InArgs._OnSelectedMediaItemsChanged;

	CommandList = MakeShared<FUICommandList>();
	BindCommands();
	
	FillMediaTree();
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 8.0f, 8.0f, 4.0f)
		[
			SAssignNew(Toolbar, SMediaProfileSourcesToolbar, InMediaProfile)
			.OnFilterChanged(this, &SMediaProfileSourcesTreeView::OnFilterChanged)
			.OnMediaSourceAdded(this, &SMediaProfileSourcesTreeView::AddNewMediaSource)
			.OnMediaOutputAdded(this, &SMediaProfileSourcesTreeView::AddNewMediaOutput)
		]
		
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(MediaTreeView, STreeView<FMediaTreeItemPtr>)
			.TreeItemsSource(&MediaTreeItems)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+SHeaderRow::Column(Column_Index)
				.DefaultLabel(LOCTEXT("IndexColumnLabel", "#"))
				.ManualWidth(20.0f)

				+SHeaderRow::Column(Column_ItemLabel)
				.DefaultLabel(LOCTEXT("ItemLabelColumnLabel", "Item Label"))
				.FillWidth(0.33f)

				+SHeaderRow::Column(Column_ItemType)
				.DefaultLabel(LOCTEXT("ItemTypeColumnLabel", "Item Type"))
				.FillWidth(0.33f)

				+SHeaderRow::Column(Column_Configuration)
				.DefaultLabel(LOCTEXT("ConfigurationColumnLabel", "Configuration"))
				.FillWidth(0.33f)			
			)
			.OnGetChildren(this, &SMediaProfileSourcesTreeView::GetTreeItemChildren)
			.OnGenerateRow(this, &SMediaProfileSourcesTreeView::GenerateTreeItemRow)
			.OnContextMenuOpening(this, &SMediaProfileSourcesTreeView::CreateContextMenu)
			.OnSelectionChanged(this, &SMediaProfileSourcesTreeView::OnMediaTreeSelectionChanged)
		]
	];

	// Expand all elements in the tree view
	for (const FMediaTreeItemPtr& TreeItem : MediaTreeItems)
	{
		MediaTreeView->SetItemExpansion(TreeItem, true);
	}
	
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SMediaProfileSourcesTreeView::OnObjectPropertyChanged);
	GEditor->RegisterForUndo(this);
}

SMediaProfileSourcesTreeView::~SMediaProfileSourcesTreeView()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	GEditor->UnregisterForUndo(this);
}

FReply SMediaProfileSourcesTreeView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (this->CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void SMediaProfileSourcesTreeView::PostUndo(bool bSuccess)
{
	// If the list of sources or outputs has changed in the media profile from the undo, update the media profile settings config
	constexpr bool bSilent = true;
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(MediaProfile->NumMediaSources(), bSilent);
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(MediaProfile->NumMediaOutputs(), bSilent);
}

void SMediaProfileSourcesTreeView::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void SMediaProfileSourcesTreeView::BindCommands()
{
	CommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::CopySelectedMedia),
		FCanExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::CanCopySelectedMedia));

	CommandList->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::CutSelectedMedia),
		FCanExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::CanCutSelectedMedia));

	CommandList->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::PasteSelectedMedia),
		FCanExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::CanPasteSelectedMedia));

	CommandList->MapAction(FGenericCommands::Get().Duplicate,
	FExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::DuplicateSelectedMedia),
	FCanExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::CanDuplicateSelectedMedia));
	
	CommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::DeleteSelectedMedia),
		FCanExecuteAction::CreateSP(this, &SMediaProfileSourcesTreeView::CanDeleteSelectedMedia));
}

void SMediaProfileSourcesTreeView::CopySelectedMedia()
{
	TArray<UObject*> ObjectsToCopy;
	for (const FMediaTreeItemPtr& SelectedItem : MediaTreeView->GetSelectedItems())
	{
		if (SelectedItem->BackingObject.IsValid())
		{
			ObjectsToCopy.Add(SelectedItem->BackingObject.Get());
		}
	}
	
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	for (UObject* Object : ObjectsToCopy)
	{
		UExporter::ExportToOutputDevice(&Context, Object, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, nullptr);
	}

	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}

bool SMediaProfileSourcesTreeView::CanCopySelectedMedia() const
{
	TArray<FMediaTreeItemPtr> SelectedItems = MediaTreeView->GetSelectedItems();
	for (const FMediaTreeItemPtr& SelectedItem : SelectedItems)
	{
		// If any of the selected media sources are copyable (are non-null media sources or outputs), allow the copy action
		if (SelectedItem->IsLeaf() && SelectedItem->BackingObject.IsValid())
		{
			return true;
		}
	}
	
	return false;
}

void SMediaProfileSourcesTreeView::CutSelectedMedia()
{
	CopySelectedMedia();
	DeleteSelectedMedia();
}

bool SMediaProfileSourcesTreeView::CanCutSelectedMedia() const
{
	return CanCopySelectedMedia() && CanDeleteSelectedMedia();
}

/** Object factory to construct a list of media sources and outputs from the clipboard */
class FMediaObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FMediaObjectTextFactory() : FCustomizableTextObjectFactory(GWarn) { }

	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		const bool bIsMediaSource = ObjectClass->IsChildOf(UMediaSource::StaticClass());
		const bool bIsMediaOutput = ObjectClass->IsChildOf(UMediaOutput::StaticClass());

		return bIsMediaSource || bIsMediaOutput;
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		if (UMediaSource* MediaSource = Cast<UMediaSource>(NewObject))
		{
			MediaSources.Add(MediaSource);
		}
		else if (UMediaOutput* MediaOutput = Cast<UMediaOutput>(NewObject))
		{
			MediaOutputs.Add(MediaOutput);
		}
	}

public:
	TArray<UMediaSource*> MediaSources;
	TArray<UMediaOutput*> MediaOutputs;
};

void SMediaProfileSourcesTreeView::PasteSelectedMedia()
{
	if (!MediaProfile.IsValid())
	{
		return;
	}
	
	FScopedTransaction PasteMediaTransaction(LOCTEXT("PasteMediaTransaction", "Paste Media"));
	MediaProfile->Modify();
	
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	FMediaObjectTextFactory Factory;
	Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardContent);

	int32 NumSourcesAdded = 0;
	for (UMediaSource* MediaSourceToCopy : Factory.MediaSources)
	{
		UMediaSource* MediaSource = DuplicateObject(MediaSourceToCopy, MediaProfile.Get());
		MediaProfile->AddMediaSource(MediaSource);
		++NumSourcesAdded;
	}

	int32 NumOutputsAdded = 0;
	for (UMediaOutput* MediaOutputToCopy : Factory.MediaOutputs)
	{
		UMediaOutput* MediaOutput = DuplicateObject(MediaOutputToCopy, MediaProfile.Get());
		MediaProfile->AddMediaOutput(MediaOutput);
		++NumOutputsAdded;
	}

	// Now, update the media profile project settings so that the proxy media slots match
	constexpr bool bSilent = true;
	if (NumSourcesAdded > 0)
	{
		GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(MediaProfile->NumMediaSources(), bSilent);
	}
	
	if (NumOutputsAdded > 0)
	{
		GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(MediaProfile->NumMediaOutputs(), bSilent);
	}
	
	MediaSourcesTreeView::TrySaveMediaProfileSettings();

	FillMediaTree();
}

bool SMediaProfileSourcesTreeView::CanPasteSelectedMedia() const
{
	// Can't paste unless the clipboard has a string in it
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	if (!ClipboardContent.IsEmpty())
	{
		FMediaObjectTextFactory Factory;
		Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardContent);
		return Factory.MediaSources.Num() > 0 || Factory.MediaOutputs.Num() > 0;
	}

	return false;
}

void SMediaProfileSourcesTreeView::DuplicateSelectedMedia()
{
	CopySelectedMedia();
	PasteSelectedMedia();
}

bool SMediaProfileSourcesTreeView::CanDuplicateSelectedMedia() const
{
	return CanCopySelectedMedia();
}

void SMediaProfileSourcesTreeView::DeleteSelectedMedia()
{
	if (!MediaProfile.IsValid())
	{
		return;
	}
	
	TArray<FMediaTreeItemPtr> SelectedItems = MediaTreeView->GetSelectedItems();
	
	TArray<int32> SourcesToRemove;
	TArray<int32> OutputsToRemove;
	for (const FMediaTreeItemPtr& SelectedItem : SelectedItems)
	{
		if (SelectedItem->BackingClass.IsValid())
		{
			if (SelectedItem->BackingClass.Get() == UMediaSource::StaticClass())
			{
				SourcesToRemove.Add(SelectedItem->Index);
			}
			else if (SelectedItem->BackingClass.Get() == UMediaOutput::StaticClass())
			{
				OutputsToRemove.Add(SelectedItem->Index);
			}
		}
	}

	if (SourcesToRemove.Num() == 0 && OutputsToRemove.Num() == 0)
	{
		return;
	}

	MediaTreeView->ClearSelection();
	
	FScopedTransaction Transaction(LOCTEXT("DeleteMediaTransaction", "Delete Media"));
	MediaProfile->Modify();

	// First, remove the media from the profile, to ensure that the correct media is moved into the correct slots
	// Reverse iterate through the indices for proper removal
	SourcesToRemove.Sort();
	for (int32 Index = SourcesToRemove.Num() - 1; Index >= 0; --Index)
	{
		OnMediaItemDeleted.ExecuteIfBound(UMediaSource::StaticClass(), SourcesToRemove[Index]);
		MediaProfile->RemoveMediaSource(SourcesToRemove[Index]);
	}

	UMediaProfileEditorCaptureSettings* MediaCaptureSettings = FMediaProfileEditor::GetMediaFrameworkCaptureSettings();

	auto RemoveMediaCaptures = [MediaCaptureSettings](UMediaOutput* MediaOutput)
	{
		if (MediaCaptureSettings && MediaOutput)
		{
			MediaCaptureSettings->ViewportCaptures.RemoveAll([MediaOutput](const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& ViewportOutputInfo)
			{
				return ViewportOutputInfo.MediaOutput == MediaOutput;
			});

			MediaCaptureSettings->RenderTargetCaptures.RemoveAll([MediaOutput](const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& RenderTargetOutputInfo)
			{
				return RenderTargetOutputInfo.MediaOutput == MediaOutput;
			});

			MediaCaptureSettings->SaveConfig();
		}
	};
	
	OutputsToRemove.Sort();
	for (int32 Index = OutputsToRemove.Num() - 1; Index >= 0; --Index)
	{
		OnMediaItemDeleted.ExecuteIfBound(UMediaOutput::StaticClass(), OutputsToRemove[Index]);
		RemoveMediaCaptures(MediaProfile->GetMediaOutput(Index));
		MediaProfile->RemoveMediaOutput(OutputsToRemove[Index]);
	}

	// Then, update the media profile project settings so that the proxy media slots match
	constexpr bool bSilent = true;
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(MediaProfile->NumMediaSources(), bSilent);
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(MediaProfile->NumMediaOutputs(), bSilent);
	
	MediaSourcesTreeView::TrySaveMediaProfileSettings();

	FillMediaTree();
}

bool SMediaProfileSourcesTreeView::CanDeleteSelectedMedia() const
{
	for (const FMediaTreeItemPtr& SelectedItem : MediaTreeView->GetSelectedItems())
	{
		if (SelectedItem->IsLeaf() && SelectedItem->BackingClass.IsValid())
		{
			return true;
		}
	}
	
	return false;
}

void SMediaProfileSourcesTreeView::AddNewMediaSource()
{
	if (MediaProfile.IsValid())
	{
		const int32 NumSources = MediaProfile->NumMediaSources();
		if (NumSources >= 16)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MaxMediaSourcesMessage", "Media profile only supports having up to 16 media sources"));
			return;
		}
		
		// TODO: Future proofing for when the media profile settings are reworked to fill from the active media profile sources/outputs instead of the other way around
		FScopedTransaction AddMediaSourceTransaction(LOCTEXT("AddMediaSourceTransaction", "Add new Media Source"));
		MediaProfile->Modify();
		MediaProfile->AddMediaSource(nullptr);

		GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(NumSources + 1);

		MediaSourcesTreeView::TrySaveMediaProfileSettings();

		TOptional<TArray<int32>> MediaSourcesToSelect = TArray<int32> { NumSources };
		TOptional<TArray<int32>> MediaOutputsToSelect = TOptional<TArray<int32>>(); 
		FillMediaTree(MediaSourcesToSelect, MediaOutputsToSelect);
	}
}

void SMediaProfileSourcesTreeView::AddNewMediaOutput()
{
	if (MediaProfile.IsValid())
	{
		const int32 NumOutputs = MediaProfile->NumMediaOutputs();
		if (NumOutputs >= 16)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MaxMediaOutputsMessage", "Media profile only supports having up to 16 media outputs"));
			return;
		}
		
		// TODO: Future proofing for when the media profile settings are reworked to fill from the active media profile sources/outputs instead of the other way around
		FScopedTransaction AddMediaOutputTransaction(LOCTEXT("AddMediaOutputTransaction", "Add new Media Output"));
		MediaProfile->Modify();
		MediaProfile->AddMediaOutput(nullptr);

		GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(NumOutputs + 1);

		MediaSourcesTreeView::TrySaveMediaProfileSettings();

		TOptional<TArray<int32>> MediaSourcesToSelect = TOptional<TArray<int32>>(); 
		TOptional<TArray<int32>> MediaOutputsToSelect = TArray<int32> { NumOutputs };
		FillMediaTree(MediaSourcesToSelect, MediaOutputsToSelect);
	}
}

void SMediaProfileSourcesTreeView::FillMediaTree(const TOptional<TArray<int32>>& InMediaSourcesToSelect, const TOptional<TArray<int32>>& InMediaOutputsToSelect)
{
	// In case a media object changed type, store the current selection through their indexes to re-select them correctly after
	// the tree has been recreated
	TArray<int32> SelectedMediaSources;
	TArray<int32> SelectedMediaOutputs;
	GetSelectedMediaIndexes(SelectedMediaSources, SelectedMediaOutputs, true);
	
	MediaTreeItems.Empty();

	if (MediaTreeView.IsValid())
	{
		MediaTreeView->ClearSelection();
	}
	
	if (MediaProfile.IsValid())
	{
		FMediaTreeItemPtr MediaSourcesTreeItem = MakeShared<FMediaTreeItem>();
		MediaSourcesTreeItem->Label = LOCTEXT("MediaSourcesTreeItemLabel", "Media Sources");
		MediaSourcesTreeItem->BackingClass = UMediaSource::StaticClass();
		
		for (int32 Index = 0; Index < MediaProfile->NumMediaSources(); ++Index)
		{
			UMediaSource* MediaSource = MediaProfile->GetMediaSource(Index);
			FMediaTreeItemPtr MediaSourceTreeItem = MakeShared<FMediaTreeItem>();
			MediaSourceTreeItem->BackingClass = UMediaSource::StaticClass();
			MediaSourceTreeItem->BackingObject = MediaSource;
			MediaSourceTreeItem->Index = Index;

			// If the media source is a proxy media source, we want to output the information of the referenced media source asset instead of the proxy
			bool bIsProxy = false;
			if (UProxyMediaSource* ProxyMediaSource = Cast<UProxyMediaSource>(MediaSource))
			{
				MediaSource = ProxyMediaSource->GetLeafMediaSource();
				bIsProxy = true;
			}

			if (MediaSource)
			{
				MediaSourceTreeItem->Label = FText::FromString(MediaProfile->GetLabelForMediaSource(Index));
				MediaSourceTreeItem->Type = MediaSource->GetClass()->GetDisplayNameText();
				MediaSourceTreeItem->Icon = FSlateIconFinder::FindIconForClass(MediaSource->GetClass()).GetIcon();
				MediaSourceTreeItem->Configuration = FText::FromString(MediaSource->GetDescriptionString());
				MediaSourceTreeItem->bFilteredOut = Toolbar.IsValid() ? !Toolbar->ItemPassesFilters(MediaSourcesTreeItem) : false;
			}
			else
			{
				MediaSourceTreeItem->Label = bIsProxy ?
					LOCTEXT("UnconfiguredProxyMediaSourceLabel", "Proxy Media Source not set") :
					LOCTEXT("NoMediaSourceSetLabel", "No Media Source Set");
			}

			MediaSourcesTreeItem->Children.Add(MediaSourceTreeItem);
		}

		MediaTreeItems.Add(MediaSourcesTreeItem);
		
		FMediaTreeItemPtr MediaOutputsTreeItem = MakeShared<FMediaTreeItem>();
		MediaOutputsTreeItem->Label = LOCTEXT("MediaOutputsTreeItemLabel", "Media Outputs");
		MediaOutputsTreeItem->BackingClass = UMediaOutput::StaticClass();
		
		for (int32 Index = 0; Index < MediaProfile->NumMediaOutputs(); ++Index)
		{
			UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(Index);
			FMediaTreeItemPtr MediaOutputTreeItem = MakeShared<FMediaTreeItem>();
			MediaOutputTreeItem->BackingClass = UMediaOutput::StaticClass();
			MediaOutputTreeItem->BackingObject = MediaOutput;
			MediaOutputTreeItem->Index = Index;

			// If the media output is a proxy media output, we want to output the information of the referenced media output asset instead of the proxy
			bool bIsProxy = false;
			if (UProxyMediaOutput* ProxyMediaOutput = Cast<UProxyMediaOutput>(MediaOutput))
			{
				MediaOutput = ProxyMediaOutput->GetLeafMediaOutput();
				bIsProxy = true;
			}

			if (MediaOutput)
			{
				MediaOutputTreeItem->Label = FText::FromString(MediaProfile->GetLabelForMediaOutput(Index));
				MediaOutputTreeItem->Type = MediaOutput->GetClass()->GetDisplayNameText();
				MediaOutputTreeItem->Icon = FSlateIconFinder::FindIconForClass(MediaOutput->GetClass()).GetIcon();
				MediaOutputTreeItem->Configuration = FText::FromString(MediaOutput->GetDescriptionString());
				MediaOutputTreeItem->bFilteredOut = Toolbar.IsValid() ? !Toolbar->ItemPassesFilters(MediaSourcesTreeItem) : false;
			}
			else
			{
				MediaOutputTreeItem->Label = bIsProxy ?
					LOCTEXT("UnconfiguredProxyMediaOutputLabel", "Proxy Media Output not set") :
					LOCTEXT("NoMediaOutputSetLabel", "No Media Output Set");
			}

			MediaOutputsTreeItem->Children.Add(MediaOutputTreeItem);
		}

		MediaTreeItems.Add(MediaOutputsTreeItem);
	}

	if (MediaTreeView.IsValid())
	{
		MediaTreeView->RequestTreeRefresh();
		
		for (const FMediaTreeItemPtr& TreeItem : MediaTreeItems)
		{
			MediaTreeView->SetItemExpansion(TreeItem, true);
		}

		if (MediaTreeItems.Num() > 0)
		{
			const TArray<int32>& SourcesToSelect = InMediaSourcesToSelect.IsSet() ? InMediaSourcesToSelect.GetValue() : SelectedMediaSources;
			for (int32 SourceIndex : SourcesToSelect)
			{
				if (SourceIndex == INDEX_NONE)
				{
					MediaTreeView->SetItemSelection(MediaTreeItems[0], true);
				}
				else if (MediaTreeItems[0]->Children.IsValidIndex(SourceIndex))
				{
					MediaTreeView->SetItemSelection(MediaTreeItems[0]->Children[SourceIndex], true);
				}
			}

			const TArray<int32>& OutputsToSelect = InMediaOutputsToSelect.IsSet() ? InMediaOutputsToSelect.GetValue() : SelectedMediaOutputs;
			for (int32 OutputIndex : OutputsToSelect)
			{
				if (OutputIndex == INDEX_NONE)
				{
					MediaTreeView->SetItemSelection(MediaTreeItems[1], true);
				}
				else if (MediaTreeItems[1]->Children.IsValidIndex(OutputIndex))
				{
					MediaTreeView->SetItemSelection(MediaTreeItems[1]->Children[OutputIndex], true);
				}
			}
		}
	}
}

void SMediaProfileSourcesTreeView::GetSelectedMediaIndexes(TArray<int32>& OutSelectedMediaSources, TArray<int32>& OutSelectedMediaOutputs, bool bIncludeGroupIndex)
{
	if (!MediaTreeView.IsValid())
	{
		return;
	}

	TArray<FMediaTreeItemPtr> SelectedItems = MediaTreeView->GetSelectedItems();

	for (const FMediaTreeItemPtr& Item : SelectedItems)
	{
		if (!Item.IsValid())
		{
			continue;
		}

		if (!Item->IsLeaf() && !bIncludeGroupIndex)
		{
			continue;
		}

		if (Item->BackingClass->IsChildOf<UMediaSource>())
		{
			OutSelectedMediaSources.Add(Item->Index);
		}
		else if (Item->BackingClass->IsChildOf<UMediaOutput>())
		{
			OutSelectedMediaOutputs.Add(Item->Index);
		}
	}
}

void SMediaProfileSourcesTreeView::GetTreeItemChildren(FMediaTreeItemPtr InTreeItem, TArray<FMediaTreeItemPtr>& OutChildren) const
{
	if (InTreeItem.IsValid())
	{
		for (FMediaTreeItemPtr& Child : InTreeItem->Children)
		{
			if (Child.IsValid() && !Child->bFilteredOut)
			{
				OutChildren.Add(Child);
			}
		}
	}
}

TSharedRef<ITableRow> SMediaProfileSourcesTreeView::GenerateTreeItemRow(FMediaTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	return SNew(SMediaTreeItemRow, InOwnerTableView, InTreeItem)
		.OnMoveMediaItem(this, &SMediaProfileSourcesTreeView::OnMoveMediaItem);
}

TSharedPtr<SWidget> SMediaProfileSourcesTreeView::CreateContextMenu()
{
	const int32 NumItems = MediaTreeView->GetNumItemsSelected();
	if (NumItems >= 1)
	{
		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		
		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

void SMediaProfileSourcesTreeView::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (!MediaProfile.IsValid())
	{
		return;
	}
	
	if (!InObject)
	{
		return;
	}

	if (MediaProfile != InObject)
	{
		// If the object isn't the media profile itself, check to see if it is one of the profile's sources or outputs,
		// as we want to update the tree view if the sources or outputs change as well
		bool bFoundMediaItem = false;
		if (UMediaSource* MediaSource = Cast<UMediaSource>(InObject))
		{
			if (MediaProfile->FindMediaSourceIndex(MediaSource) != INDEX_NONE)
			{
				bFoundMediaItem = true;
			}
		}

		if (UMediaOutput* MediaOutput = Cast<UMediaOutput>(InObject))
		{
			if (MediaProfile->FindMediaOutputIndex(MediaOutput) != INDEX_NONE)
			{
				bFoundMediaItem = true;
			}
		}
		
		if (!bFoundMediaItem)
		{
			return;
		}
	}
	
	FillMediaTree();
}

void SMediaProfileSourcesTreeView::OnFilterChanged()
{
	for (FMediaTreeItemPtr& TreeItem : MediaTreeItems)
	{
		for (FMediaTreeItemPtr& ChildItem : TreeItem->Children)
		{
			ChildItem->bFilteredOut = !Toolbar->ItemPassesFilters(ChildItem);
		}
	}

	if (MediaTreeView.IsValid())
	{
		MediaTreeView->RequestTreeRefresh();
		
		for (const FMediaTreeItemPtr& TreeItem : MediaTreeItems)
		{
			MediaTreeView->SetItemExpansion(TreeItem, true);
		}
	}
}

void SMediaProfileSourcesTreeView::OnMoveMediaItem(TSharedPtr<FMediaTreeItem> InTreeItem, int32 InIndex)
{
	if (!MediaProfile.IsValid())
	{
		return;
	}
	
	const bool bIsMediaSource = InTreeItem->BackingClass->IsChildOf<UMediaSource>();
	const FText TransactionText = bIsMediaSource ? LOCTEXT("MoveMediaSourceTransaction", "Move Media Source") : LOCTEXT("MoveMediaOutputTransaction", "Move Media Output");
	
	FScopedTransaction MoveMediaItemTransaction(TransactionText);
	MediaProfile->Modify();

	bool bSuccessful = false;
	if (bIsMediaSource)
	{
		bSuccessful =MediaProfile->MoveMediaSource(InTreeItem->Index, InIndex);
	}
	else
	{
		bSuccessful = MediaProfile->MoveMediaOutput(InTreeItem->Index, InIndex);
	}

	FillMediaTree();

	// Update the selection to select the newly moved item
	if (MediaTreeView.IsValid() && bSuccessful)
	{
		MediaTreeView->ClearSelection();
		if (bIsMediaSource)
		{
			MediaTreeView->SetItemSelection(MediaTreeItems[0]->Children[InIndex], true);
		}
		else
		{
			MediaTreeView->SetItemSelection(MediaTreeItems[1]->Children[InIndex], true);
		}
	}
}

void SMediaProfileSourcesTreeView::OnMediaTreeSelectionChanged(TSharedPtr<FMediaTreeItem> SelectedItem, ESelectInfo::Type SelectionType)
{
	TArray<int32> SelectedMediaSources;
	TArray<int32> SelectedMediaOutputs;
	GetSelectedMediaIndexes(SelectedMediaSources, SelectedMediaOutputs);
	
	OnSelectedMediaItemsChanged.ExecuteIfBound(SelectedMediaSources, SelectedMediaOutputs);
}

#undef LOCTEXT_NAMESPACE
