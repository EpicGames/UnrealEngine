// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Misc/FilterCollection.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class SMediaProfileSourcesToolbar;
class FUICommandList;
class UMediaProfile;

/**
 * A tree view that displays a media profile's source and outputs
 */
class SMediaProfileSourcesTreeView : public SCompoundWidget, public FEditorUndoClient
{
public:
	DECLARE_DELEGATE_TwoParams(FOnMediaItemDeleted, UClass* /* MediaItemType */, int32 /* MediaItemIndex */)
	DECLARE_DELEGATE_TwoParams(FOnSelectedMediaItemsChanged, const TArray<int32>& /* SelectedMediaSources */, const TArray<int32>& /* SelectedMediaOutputs */);
	
private:
	/** Struct to store information for each element displayed in the tree view */
	struct FMediaTreeItem
	{
		int32 Index = INDEX_NONE;
		FText Label = FText::GetEmpty();
		FText Type = FText::GetEmpty();
		FText Configuration = FText::GetEmpty();
		const FSlateBrush* Icon = nullptr;
		bool bFilteredOut = false;
		
		TArray<TSharedPtr<FMediaTreeItem>> Children;
		TWeakObjectPtr<UClass> BackingClass = nullptr;
		TWeakObjectPtr<UObject> BackingObject = nullptr;
		
		bool IsLeaf() const { return Index != INDEX_NONE; }
	};
	typedef TSharedPtr<FMediaTreeItem> FMediaTreeItemPtr;

	// Names of the columns within the tree view
	static const FLazyName Column_Index;
	static const FLazyName Column_ItemLabel;
	static const FLazyName Column_ItemType;
	static const FLazyName Column_Configuration;
	
public:
	SLATE_BEGIN_ARGS(SMediaProfileSourcesTreeView) {}
		SLATE_EVENT(FOnMediaItemDeleted, OnMediaItemDeleted)
		SLATE_EVENT(FOnSelectedMediaItemsChanged, OnSelectedMediaItemsChanged)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, UMediaProfile* InMediaProfile);
	virtual ~SMediaProfileSourcesTreeView() override;

	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	// End of SWidget interface
	
	// FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient
	
private:
	/** Initializes and binds the command list to event handlers */
	void BindCommands();

	// Tree item context menu handlers
	void CopySelectedMedia();
	bool CanCopySelectedMedia() const;
	void CutSelectedMedia();
	bool CanCutSelectedMedia() const;
	void PasteSelectedMedia();
	bool CanPasteSelectedMedia() const;
	void DuplicateSelectedMedia();
	bool CanDuplicateSelectedMedia() const;
	void DeleteSelectedMedia();
	bool CanDeleteSelectedMedia() const;

	/** Adds a new empty media source to the media profile */
	void AddNewMediaSource();

	/** Adds a new empty media output to the media profile */
	void AddNewMediaOutput();

	/** Recreates the tree view's source list from the current media profile's list of sources and outputs */
	void FillMediaTree(const TOptional<TArray<int32>>& InMediaSourcesToSelect = TOptional<TArray<int32>>(), const TOptional<TArray<int32>>& InMediaOutputsToSelect = TOptional<TArray<int32>>());

	/** Converts the selected tree items to a list of indexes of the selected media sources and outputs in the media profile */
	void GetSelectedMediaIndexes(TArray<int32>& OutSelectedMediaSources, TArray<int32>& OutSelectedMediaOutputs, bool bIncludeGroupIndex = false);
	
	// TreeView event handlers
	void GetTreeItemChildren(FMediaTreeItemPtr InTreeItem, TArray<FMediaTreeItemPtr>& OutChildren) const;
	TSharedRef<ITableRow> GenerateTreeItemRow(FMediaTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& InOwnerTableView);
	TSharedPtr<SWidget> CreateContextMenu();

	/** Raised when an object property is changed. Used to monitor changes to the media profile and update the media tree view accordingly */
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	/** Raised when the filter has changed in the toolbar, updates the tree view's source list to show only filtered items */
	void OnFilterChanged();

	/** Raised when a media tree item is moved */
	void OnMoveMediaItem(TSharedPtr<FMediaTreeItem> InTreeItem, int32 InIndex);
	
	void OnMediaTreeSelectionChanged(TSharedPtr<FMediaTreeItem> SelectedItem, ESelectInfo::Type SelectionType);
	
private:
	TArray<FMediaTreeItemPtr> MediaTreeItems;
	TSharedPtr<STreeView<FMediaTreeItemPtr>> MediaTreeView;
	TSharedPtr<FUICommandList> CommandList;
	TWeakObjectPtr<UMediaProfile> MediaProfile;

	TSharedPtr<SMediaProfileSourcesToolbar> Toolbar;

	FOnMediaItemDeleted OnMediaItemDeleted;
	FOnSelectedMediaItemsChanged OnSelectedMediaItemsChanged;
	
	friend class SMediaTreeItemRow;
	friend class SMediaProfileSourcesToolbar;
};