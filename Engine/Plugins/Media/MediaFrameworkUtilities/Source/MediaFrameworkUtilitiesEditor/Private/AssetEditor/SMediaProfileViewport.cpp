// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaProfileViewport.h"

#include "Editor.h"
#include "LevelViewportActions.h"
#include "SMediaProfileMediaItemDisplay.h"
#include "ToolMenus.h"
#include "Framework/Application/SlateApplication.h"
#include "Profile/MediaProfile.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaProfileViewport"

void SMediaProfileViewport::Construct(const FArguments& InArgs, const TSharedPtr<FMediaProfileEditor> InOwningEditor)
{
	MediaProfileEditor = InOwningEditor;

	BindCommands();
	
	ChildSlot
	[
		SAssignNew(LayoutContainer, SOverlay)
	];
}

void SMediaProfileViewport::SetSelectedMediaItems(const TArray<int32>& InSelectedMediaSourceIndices, const TArray<int32>& InSelectedMediaOutputIndices)
{
	UMediaProfile* MediaProfile = nullptr;
	if (MediaProfileEditor.IsValid())
	{
		MediaProfile = MediaProfileEditor.Pin()->GetMediaProfile();
	}
	
	if (!MediaProfile)
	{
		return;
	}
	
	TArray<int32> MediaSourceIndices = InSelectedMediaSourceIndices;
	TArray<int32> MediaOutputIndices = InSelectedMediaOutputIndices;
	
	// If a selected media source or media output is already being displayed in the panel contents as a locked panel,
	// don't add a new panel for it, only configure sources and outputs that aren't already being displayed
	for (const FPanelContent& Panel : PanelContents)
	{
		if (Panel.bIsLocked)
		{
			if (Panel.MediaType->IsChildOf(UMediaSource::StaticClass()))
			{
				MediaSourceIndices.Remove(Panel.MediaItemIndex);
			}
			else if (Panel.MediaType->IsChildOf(UMediaOutput::StaticClass()))
			{
				MediaOutputIndices.Remove(Panel.MediaItemIndex);
			}
		}
	}

	constexpr int32 MaxNumPanels = 4;
	const int NumMediaItems = MediaSourceIndices.Num() + MediaOutputIndices.Num();
	int32 PanelIndex = 0;
	
	for (int32 Index = 0; Index < NumMediaItems; ++Index)
	{
		while (PanelIndex < PanelContents.Num() && PanelContents[PanelIndex].bIsLocked)
		{
			++PanelIndex;
		}
		
		if (PanelIndex >= MaxNumPanels)
		{
			break;
		}

		const int32 MediaItemIndex = Index < MediaSourceIndices.Num() ? MediaSourceIndices[Index] : MediaOutputIndices[Index - MediaSourceIndices.Num()];
		
		UClass* MediaType;
		if (Index < MediaSourceIndices.Num())
		{
			if (UMediaSource* MediaSource = MediaProfile->GetMediaSource(MediaItemIndex))
			{
				MediaType = MediaSource->GetClass();
			}
			else
			{
				MediaType = UMediaSource::StaticClass();
			}
		}
		else
		{
			if (UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(MediaItemIndex))
			{
				MediaType = MediaOutput->GetClass();
			}
			else
			{
				MediaType = UMediaOutput::StaticClass();
			}
		}
		
		if (PanelIndex >= PanelContents.Num())
		{
			FPanelContent NewPanel;
			NewPanel.MediaItemIndex = MediaItemIndex;
			NewPanel.MediaType = MediaType;
			NewPanel.Widget = nullptr;
			PanelContents.Add(NewPanel);
		}
		else
		{
			PanelContents[PanelIndex].MediaItemIndex = MediaItemIndex;
			PanelContents[PanelIndex].MediaType = MediaType;
			PanelContents[PanelIndex].Widget = nullptr;
		}

		++PanelIndex;
	}

	// Clear out or remove any panel content beyond the panels that are set or locked
	bool bFoundLockedPanel = false;
	for (int32 Index = FMath::Min(PanelContents.Num(), MaxNumPanels) - 1; Index >= PanelIndex; --Index)
	{
		if (PanelContents[Index].bIsLocked)
		{
			bFoundLockedPanel = true;
			continue;
		}

		// If there is a locked panel after this panel, simply clear the panel's contents.
		// Otherwise, remove the panel
		if (bFoundLockedPanel)
		{
			PanelContents[Index].MediaItemIndex = INDEX_NONE;
			PanelContents[Index].MediaType = nullptr;
			PanelContents[PanelIndex].Widget = nullptr;
		}
		else
		{
			PanelContents.RemoveAt(Index);
		}
	}
	
	CurrentLayout = PanelContents.Num() >= 4 ? EViewportLayout::QuadPanel : (EViewportLayout)(PanelContents.Num() - 1);
	CurrentOrientation = EViewportOrientation::Left;
	MaximizedPanelIndex = INDEX_NONE;
	
	UpdateDisplays();
}

void SMediaProfileViewport::ForceClearMediaItem(UClass* InMediaType, int32 InMediaItemIndex)
{
	for (FPanelContent& PanelContent : PanelContents)
	{
		if (PanelContent.MediaType && PanelContent.MediaType->IsChildOf(InMediaType) && PanelContent.MediaItemIndex == InMediaItemIndex)
		{
			PanelContent.MediaType = nullptr;
			PanelContent.MediaItemIndex = INDEX_NONE;
			PanelContent.bIsLocked = false;
		}
	}
	
	UpdateDisplays();
}

void SMediaProfileViewport::SetPanelContents(int32 InPanelIndex, UClass* InMediaItemClass, int32 InMediaItemIndex, bool bRefreshDisplay)
{
	if (!PanelContents.IsValidIndex(InPanelIndex))
	{
		return;
	}
	
	PanelContents[InPanelIndex].MediaItemIndex = InMediaItemIndex;
	PanelContents[InPanelIndex].MediaType = InMediaItemClass;

	if (bRefreshDisplay)
	{
		PanelContents[InPanelIndex].Widget = nullptr;
		UpdateDisplays();
	}
}

void SMediaProfileViewport::SetPanelLocked(int32 InPanelIndex, bool bInIsLocked)
{
	if (!PanelContents.IsValidIndex(InPanelIndex))
	{
		return;
	}

	PanelContents[InPanelIndex].bIsLocked = bInIsLocked;
}

bool SMediaProfileViewport::IsPanelLocked(int32 InPanelIndex) const
{
	if (!PanelContents.IsValidIndex(InPanelIndex))
	{
		return false;
	}

	return PanelContents[InPanelIndex].bIsLocked;
}

bool SMediaProfileViewport::CanMaximizePanel(int32 InPanelIndex) const
{
	return MaximizedPanelIndex == INDEX_NONE &&
		PanelContents.Num() > 1 &&
		PanelContents.IsValidIndex(InPanelIndex);
}

void SMediaProfileViewport::MaximizePanel(int32 InPanelIndex)
{
	MaximizedPanelIndex = InPanelIndex;

	SavedLayout = CurrentLayout;
	SavedOrientation = CurrentOrientation;
	CurrentLayout = EViewportLayout::OnePanel;
	CurrentOrientation = EViewportOrientation::Left;
	
	UpdateDisplays();

	// Set keyboard focus on the panel widget after it has been maximized
	// so that keyboard shortcuts still work for that panel without the user having to click on it again
	if (PanelContents.IsValidIndex(MaximizedPanelIndex))
	{
		FSlateApplication::Get().ClearKeyboardFocus();
		FSlateApplication::Get().SetKeyboardFocus(PanelContents[MaximizedPanelIndex].Widget, EFocusCause::SetDirectly);
	}
}

bool SMediaProfileViewport::IsPanelMaximized(int32 InPanelIndex) const
{
	return MaximizedPanelIndex != INDEX_NONE && MaximizedPanelIndex == InPanelIndex;
}

bool SMediaProfileViewport::CanRestorePreviousLayout() const
{
	return MaximizedPanelIndex != INDEX_NONE &&
		PanelContents.Num() > 1;
}

void SMediaProfileViewport::RestorePreviousLayout()
{
	const int32 PreviousMaximizedPanelIndex = MaximizedPanelIndex;
	MaximizedPanelIndex = INDEX_NONE;
	CurrentLayout = SavedLayout;
	CurrentOrientation = SavedOrientation;

	UpdateDisplays();

	// Reset keyboard focus back the panel widget after it has been repositioned in the panel layout
	// so that keyboard shortcuts still work for that panel without the user having to click on it again
	if (PanelContents.IsValidIndex(PreviousMaximizedPanelIndex))
	{
		FSlateApplication::Get().ClearKeyboardFocus();
		FSlateApplication::Get().SetKeyboardFocus(PanelContents[PreviousMaximizedPanelIndex].Widget);
	}
}

void SMediaProfileViewport::SetImmersivePanel()
{
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (!ParentWindow.IsValid())
	{
		return;
	}
	
	// If there are no panel contents, don't allow immersive since there is no viewport toolbar that allows users to exit immersive
	if (PanelContents.IsEmpty())
	{
		return;
	}

	bImmersive = true;
	ParentWindow->SetFullWindowOverlayContent(LayoutContainer);

	// Reset keyboard focus back the panel widget after it has been repositioned in the panel layout
	// so that keyboard shortcuts still work for that panel without the user having to click on it again
	FSlateApplication::Get().ClearKeyboardFocus();
	FSlateApplication::Get().SetKeyboardFocus(PanelContents[0].Widget);
}

bool SMediaProfileViewport::IsPanelImmersive() const
{
	return bImmersive;
}

void SMediaProfileViewport::ClearImmersivePanel()
{
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (!ParentWindow.IsValid())
	{
		return;
	}

	bImmersive = false;
	ParentWindow->SetFullWindowOverlayContent(nullptr);

	if (!PanelContents.IsEmpty())
	{
		// Reset keyboard focus back the panel widget after it has been repositioned in the panel layout
		// so that keyboard shortcuts still work for that panel without the user having to click on it again
		FSlateApplication::Get().ClearKeyboardFocus();
		FSlateApplication::Get().SetKeyboardFocus(PanelContents[0].Widget);
	}
}

void SMediaProfileViewport::SaveLayout(FMediaProfileEditorLayoutSettings& InOutLayoutSettings)
{
	UMediaProfile* MediaProfile = nullptr;
	if (MediaProfileEditor.IsValid())
	{
		MediaProfile = MediaProfileEditor.Pin()->GetMediaProfile();
	}
	
	if (!MediaProfile)
	{
		return;
	}
	
	InOutLayoutSettings.ViewportLayout = (uint8)CurrentLayout;
	InOutLayoutSettings.ViewportOrientation = (uint8)CurrentOrientation;
	
	InOutLayoutSettings.Panels.Empty();
	for (const FPanelContent& Panel : PanelContents)
	{
		FMediaProfileEditorLayoutPanelSettings PanelSettings;
		if (Panel.MediaType->IsChildOf(UMediaSource::StaticClass()))
		{
			PanelSettings.MediaItemName = MediaProfile->GetLabelForMediaSource(Panel.MediaItemIndex);
			PanelSettings.bMediaSource = true;
		}
		else
		{
			PanelSettings.MediaItemName = MediaProfile->GetLabelForMediaOutput(Panel.MediaItemIndex);
			PanelSettings.bMediaSource = false;
		}
		
		InOutLayoutSettings.Panels.Add(PanelSettings);
	}
}

void SMediaProfileViewport::LoadLayout(const FMediaProfileEditorLayoutSettings& InLayoutSettings)
{
	UMediaProfile* MediaProfile = nullptr;
	if (MediaProfileEditor.IsValid())
	{
		MediaProfile = MediaProfileEditor.Pin()->GetMediaProfile();
	}
	
	if (!MediaProfile)
	{
		return;
	}
	
	CurrentLayout = (EViewportLayout)InLayoutSettings.ViewportLayout;
	CurrentOrientation = (EViewportOrientation)InLayoutSettings.ViewportOrientation;
	
	PanelContents.Empty();
	for (const FMediaProfileEditorLayoutPanelSettings& PanelSettings : InLayoutSettings.Panels)
	{
		FPanelContent Panel;
		
		if (PanelSettings.bMediaSource)
		{
			Panel.MediaItemIndex = MediaProfile->FindMediaSourceIndexByLabel(PanelSettings.MediaItemName);

			if (UMediaSource* MediaSource = MediaProfile->GetMediaSource(Panel.MediaItemIndex))
			{
				Panel.MediaType = MediaSource->GetClass();
			}
			else
			{
				Panel.MediaType = UMediaSource::StaticClass();
			}
		}
		else
		{
			Panel.MediaItemIndex = MediaProfile->FindMediaOutputIndexByLabel(PanelSettings.MediaItemName);

			if (UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(Panel.MediaItemIndex))
			{
				Panel.MediaType = MediaOutput->GetClass();
			}
			else
			{
				Panel.MediaType = UMediaOutput::StaticClass();
			}
		}

		Panel.bIsLocked = true;
		
		PanelContents.Add(Panel);
	}
	
	UpdateDisplays();
}

void SMediaProfileViewport::ClearLayout()
{
	CurrentLayout = EViewportLayout::OnePanel;
	CurrentOrientation = EViewportOrientation::Left;
	
	PanelContents.Empty();

	UpdateDisplays();
}

FReply SMediaProfileViewport::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		Reply = FReply::Handled();
	}

	return Reply;
}

void SMediaProfileViewport::BindCommands()
{
	CommandList = MakeShared<FUICommandList>();

	const FLevelViewportCommands& Commands = FLevelViewportCommands::Get();

	CommandList->MapAction(
		Commands.ViewportConfig_OnePane,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::OnePanel, EViewportOrientation::Left),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::OnePanel, EViewportOrientation::Left));

	CommandList->MapAction(
		Commands.ViewportConfig_TwoPanesH,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::TwoPanel, EViewportOrientation::Left),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::TwoPanel, EViewportOrientation::Left));
		
	CommandList->MapAction(
		Commands.ViewportConfig_TwoPanesV,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::TwoPanel, EViewportOrientation::Top),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::TwoPanel, EViewportOrientation::Top));

	CommandList->MapAction(
		Commands.ViewportConfig_ThreePanesLeft,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::ThreePanel, EViewportOrientation::Left),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::ThreePanel, EViewportOrientation::Left));

	CommandList->MapAction(
		Commands.ViewportConfig_ThreePanesTop,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::ThreePanel, EViewportOrientation::Top),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::ThreePanel, EViewportOrientation::Top));

	CommandList->MapAction(
		Commands.ViewportConfig_ThreePanesRight,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::ThreePanel, EViewportOrientation::Right),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::ThreePanel, EViewportOrientation::Right));

	CommandList->MapAction(
		Commands.ViewportConfig_ThreePanesBottom,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::ThreePanel, EViewportOrientation::Bottom),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::ThreePanel, EViewportOrientation::Bottom));

	CommandList->MapAction(
		Commands.ViewportConfig_FourPanesLeft,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::OneAndThreePanel, EViewportOrientation::Left),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::OneAndThreePanel, EViewportOrientation::Left));

	CommandList->MapAction(
		Commands.ViewportConfig_FourPanesTop,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::OneAndThreePanel, EViewportOrientation::Top),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::OneAndThreePanel, EViewportOrientation::Top));

	CommandList->MapAction(
		Commands.ViewportConfig_FourPanesRight,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::OneAndThreePanel, EViewportOrientation::Right),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::OneAndThreePanel, EViewportOrientation::Right));

	CommandList->MapAction(
		Commands.ViewportConfig_FourPanesBottom,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::OneAndThreePanel, EViewportOrientation::Bottom),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::OneAndThreePanel, EViewportOrientation::Bottom));

	CommandList->MapAction(
		Commands.ViewportConfig_FourPanes2x2,
		FExecuteAction::CreateSP(this, &SMediaProfileViewport::SetViewportLayout, EViewportLayout::QuadPanel, EViewportOrientation::Left),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SMediaProfileViewport::IsViewportLayoutSet, EViewportLayout::QuadPanel, EViewportOrientation::Left));
}

void SMediaProfileViewport::UpdateDisplays()
{
	LayoutContainer->ClearChildren();
	
	UMediaProfile* MediaProfile = nullptr;
	if (MediaProfileEditor.IsValid())
	{
		MediaProfile = MediaProfileEditor.Pin()->GetMediaProfile();
	}
	
	if (!MediaProfile)
	{
		return;
	}
	
	if (PanelContents.Num() <= 0)
	{
		LayoutContainer->AddSlot()
		[
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.BorderBackgroundColor(FLinearColor::Black)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoMediaItemSelectedLabel", "No media item selected"))
			]
		];
		
		return;
	}
	
	LayoutContainer->AddSlot()
	[
		ConstructViewportLayout()
	];
}

TSharedRef<SWidget> SMediaProfileViewport::ConstructPanelWidget(int32 InPanelIndex)
{
	if (InPanelIndex >= PanelContents.Num())
	{
		// If a panel is being requested that doesn't have any stored contents, fill the panel contents array
		// with empty panels up to that index
		PanelContents.AddDefaulted(InPanelIndex + 1 - PanelContents.Num());
	}

	FPanelContent& PanelContent = PanelContents[InPanelIndex];
	if (PanelContent.Widget.IsValid())
	{
		return PanelContent.Widget.ToSharedRef();
	}
	
	if (PanelContent.MediaType && PanelContent.MediaType->IsChildOf<UMediaSource>())
	{
		return SAssignNew(PanelContent.Widget, SMediaProfileMediaSourceDisplay, SharedThis(this))
			.MediaProfileEditor(MediaProfileEditor)
			.PanelIndex(InPanelIndex)
			.MediaItemIndex(PanelContent.MediaItemIndex);
	}

	if (PanelContent.MediaType && PanelContent.MediaType->IsChildOf<UMediaOutput>())
	{
		return SAssignNew(PanelContent.Widget, SMediaProfileMediaOutputDisplay, SharedThis(this))
			.MediaProfileEditor(MediaProfileEditor)
			.PanelIndex(InPanelIndex)
			.MediaItemIndex(PanelContent.MediaItemIndex);
	}

	return SAssignNew(PanelContent.Widget, SMediaProfileDummyDisplay, SharedThis(this))
		.MediaProfileEditor(MediaProfileEditor)
		.PanelIndex(InPanelIndex);
}

TSharedRef<SWidget> SMediaProfileViewport::ConstructViewportLayout()
{
	UMediaProfile* MediaProfile = nullptr;
	if (MediaProfileEditor.IsValid())
	{
		MediaProfile = MediaProfileEditor.Pin()->GetMediaProfile();
	}
	
	if (!MediaProfile)
	{
		return SNullWidget::NullWidget;
	}
	
	if (CurrentLayout == EViewportLayout::OnePanel)
	{
		const bool bIsMaximizingPanel = MaximizedPanelIndex != INDEX_NONE && PanelContents.IsValidIndex(MaximizedPanelIndex);
		return ConstructPanelWidget(bIsMaximizingPanel ? MaximizedPanelIndex : 0);
	}

	if (CurrentLayout == EViewportLayout::TwoPanel)
	{
		const bool bIsHorizontal = CurrentOrientation == EViewportOrientation::Left || CurrentOrientation == EViewportOrientation::Right;
		return SNew(SSplitter)
			.Orientation(bIsHorizontal ? Orient_Horizontal : Orient_Vertical)

			+SSplitter::Slot()
			[
				ConstructPanelWidget(0)
			]

			+SSplitter::Slot()
			[
				ConstructPanelWidget(1)
			];
	}

	if (CurrentLayout == EViewportLayout::ThreePanel)
	{
		const bool bIsHorizontal = CurrentOrientation == EViewportOrientation::Left || CurrentOrientation == EViewportOrientation::Right;
		const bool bIsTopLeft = CurrentOrientation == EViewportOrientation::Left || CurrentOrientation == EViewportOrientation::Top;
		TSharedRef<SSplitter> SecondarySplitter = SNew(SSplitter)
			.Orientation(bIsHorizontal ? Orient_Vertical : Orient_Horizontal)
			
			+SSplitter::Slot()
			[
				ConstructPanelWidget(1)
			]

			+SSplitter::Slot()
			[
				ConstructPanelWidget(2)
			];

		return SNew(SSplitter)
			.Orientation(bIsHorizontal ? Orient_Horizontal : Orient_Vertical)

			+SSplitter::Slot()
			[
				bIsTopLeft ? ConstructPanelWidget(0) : SecondarySplitter
			]

			+SSplitter::Slot()
			[
				bIsTopLeft ? SecondarySplitter : ConstructPanelWidget(0)
			];
	}

	if (CurrentLayout == EViewportLayout::OneAndThreePanel)
	{
		const bool bIsHorizontal = CurrentOrientation == EViewportOrientation::Left || CurrentOrientation == EViewportOrientation::Right;
		const bool bIsTopLeft = CurrentOrientation == EViewportOrientation::Left || CurrentOrientation == EViewportOrientation::Top;
		TSharedRef<SSplitter> SecondarySplitter = SNew(SSplitter)
			.Orientation(bIsHorizontal ? Orient_Vertical : Orient_Horizontal)
			
			+SSplitter::Slot()
			[
				ConstructPanelWidget(1)
			]

			+SSplitter::Slot()
			[
				ConstructPanelWidget(2)
			]

			+SSplitter::Slot()
			[
				ConstructPanelWidget(3)
			];;

		return SNew(SSplitter)
			.Orientation(bIsHorizontal ? Orient_Horizontal : Orient_Vertical)

			+SSplitter::Slot()
			[
				bIsTopLeft ? ConstructPanelWidget(0) : SecondarySplitter
			]

			+SSplitter::Slot()
			[
				bIsTopLeft ? SecondarySplitter : ConstructPanelWidget(0)
			];
	}

	if (CurrentLayout == EViewportLayout::QuadPanel)
	{
		return SNew(SSplitter2x2)
			.TopLeft()
			[
				ConstructPanelWidget(0)
			]
			.BottomLeft()
			[
				ConstructPanelWidget(2)
			]
			.TopRight()
			[
				ConstructPanelWidget(1)
			]
			.BottomRight()
			[
				ConstructPanelWidget(3)
			];
	}

	return SNullWidget::NullWidget;
}

void SMediaProfileViewport::SetViewportLayout(EViewportLayout InViewportLayout, EViewportOrientation InViewportOrientation)
{
	CurrentLayout = InViewportLayout;
	CurrentOrientation = InViewportOrientation;
	MaximizedPanelIndex = INDEX_NONE;
	
	UpdateDisplays();

	FSlateApplication::Get().DismissAllMenus();
}

bool SMediaProfileViewport::IsViewportLayoutSet(EViewportLayout InViewportLayout, EViewportOrientation InViewportOrientation) const
{
	return CurrentLayout == InViewportLayout && CurrentOrientation == InViewportOrientation;
}

#undef LOCTEXT_NAMESPACE