// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolMenuEntry.h"
#include "Widgets/SCompoundWidget.h"

struct FMediaProfileEditorLayoutSettings;
class FMediaProfileEditor;
class SOverlay;
class FUICommandList;
class UMediaProfile;

/** Viewport that displays the input or output from selected media sources and outputs */
class SMediaProfileViewport : public SCompoundWidget
{
public:
	/** Indicates how to lay out the panels of the viewport */
	enum class EViewportLayout
	{
		/** Single panel layout */
		OnePanel,

		/** Two panel layout split either horizontally or vertically */
		TwoPanel,

		/** Three panel layout where one half is a full panel and the other half is split between two panels */
		ThreePanel,

		/** Four panel layout where one half is a full panel and the other half is split between three panels */
		OneAndThreePanel,

		/** A 2x2 panel split */
		QuadPanel
	};

	/** Indicates how to orient the primary panel in panel layouts that can have multiple orientations (e.g. three panel) */
	enum class EViewportOrientation
	{
		Left,
		Top,
		Right,
		Bottom
	};
	
public:
	SLATE_BEGIN_ARGS(SMediaProfileViewport) {}
	SLATE_END_ARGS()
		
	void Construct(const FArguments& InArgs, const TSharedPtr<FMediaProfileEditor> InOwningEditor);
	
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	
	/** Displays the selected media items in a viewport panel arrangement */
	void SetSelectedMediaItems(const TArray<int32>& InSelectedMediaSourceIndices, const TArray<int32>& InSelectedMediaOutputIndices);

	/** Removes all panels that contain the specified media item, even if the panels are locked */
	void ForceClearMediaItem(UClass* InMediaType, int32 InMediaItemIndex);

	/** Sets the contents of the specified panel to the specified media type and media index */
	void SetPanelContents(int32 InPanelIndex, UClass* InMediaItemClass, int32 InMediaItemIndex, bool bRefreshDisplay);
	
	/** Sets the lock status the specified panel */
	void SetPanelLocked(int32 InPanelIndex, bool bInIsLocked);
	
	/** Gets whether the specified panel is locked or not */
	bool IsPanelLocked(int32 InPanelIndex) const;
	
	/** Gets whether the specified panel can be maximized right now or not*/
	bool CanMaximizePanel(int32 InPanelIndex) const;
	
	/** Maximizes the contents in the specified panel index, moving them to the first panel and changing the arrangement to single panel */
	void MaximizePanel(int32 InPanelIndex);

	/** Gets whether the specified panel is currently maximized */
	bool IsPanelMaximized(int32 InPanelIndex) const;
	
	/** Gets whether there is a previous layout that can be restored */
	bool CanRestorePreviousLayout() const;
	
	/** Restores the previous multi-panel layout from a maximized state, if one is saved */
	void RestorePreviousLayout();

	/** Sets the panel layout to display as immersive fullscreen */
	void SetImmersivePanel();

	/** Gets whether the panel layout is being displayed in immersive view */
	bool IsPanelImmersive() const;
	
	/** Clears any immersive panel being displayed */
	void ClearImmersivePanel();

	/** Saves the current layout of the viewport's panels into the specified named layout */
	void SaveLayout(FMediaProfileEditorLayoutSettings& InOutLayoutSettings);

	/** Loads the panel configuration from the specified named layout into the viewport's panels */
	void LoadLayout(const FMediaProfileEditorLayoutSettings& InLayoutSettings);

	/** Clears the current panel layout back to default */
	void ClearLayout();

	/** Gets the viewport's command list */
	TSharedPtr<FUICommandList> GetCommandList() { return CommandList; }
	
private:
	/** Binds commands in this widget's command list to actions */
	void BindCommands();
	
	/** Updates the current media item displays to match the selected media items  */
	void UpdateDisplays();

	/** Constructs the widget to display the contents of the specified panel */
	TSharedRef<SWidget> ConstructPanelWidget(int32 InPanelIndex);

	/** Creates and lays out the panels for the viewport based on the currently set layout, orientation, and panel contents */
	TSharedRef<SWidget> ConstructViewportLayout();

	/** Sets the viewport panel layout to the specified layout and orientation */
	void SetViewportLayout(EViewportLayout InViewportLayout, EViewportOrientation InViewportOrientation);
	
	/** Returns whether the current viewport panel layout matches the specified layout and orientation */
	bool IsViewportLayoutSet(EViewportLayout InViewportLayout, EViewportOrientation InViewportOrientation) const;
	
private:
	/** Media profile editor that owns this viewport */
	TWeakPtr<FMediaProfileEditor> MediaProfileEditor;

	/** Container that holds the actual panel layout widgets */
	TSharedPtr<SOverlay> LayoutContainer;

	/** Stores the content of a displayed viewport panel */
	struct FPanelContent
	{
		UClass* MediaType = nullptr;
		int32 MediaItemIndex = INDEX_NONE;
		bool bIsLocked = false;
		TSharedPtr<SWidget> Widget;
	};

	/** The content of the currently displayed viewport panels */
	TArray<FPanelContent> PanelContents;

	/** Command list for the viewport tab */
	TSharedPtr<FUICommandList> CommandList;

	/** Currently active viewport layout */
	EViewportLayout CurrentLayout = EViewportLayout::OnePanel;
	
	/** Currently active viewport orientation */
	EViewportOrientation CurrentOrientation = EViewportOrientation::Left;
	
	/** Stores the previously used layout to restore if toggling out of maximized panel */
	EViewportLayout SavedLayout = EViewportLayout::OnePanel;

	/** Stores the previously used orientation to restore if toggling out of maximized panel */
	EViewportOrientation SavedOrientation = EViewportOrientation::Left;

	/** When maximizing from a multi-panel layout, this is the panel being maximized */
	int32 MaximizedPanelIndex = INDEX_NONE;

	/** Whether the panel layout is being displayed in immersive fullscreen */
	bool bImmersive = false;
};