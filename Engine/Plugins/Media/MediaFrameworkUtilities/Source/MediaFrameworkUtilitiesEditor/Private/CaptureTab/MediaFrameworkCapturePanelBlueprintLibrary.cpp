// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureTab/MediaFrameworkCapturePanelBlueprintLibrary.h"
#include "CaptureTab/SMediaFrameworkCapture.h"
#include "GameFramework/Actor.h" // IWYU pragma: keep
#include "MediaFrameworkUtilitiesEditorModule.h"

#define LOCTEXT_NAMESPACE "MediaFrameworkUtilities"

void UMediaFrameworkCapturePanel::StartCapture()
{
	TSharedPtr<SMediaFrameworkCapture> Panel = SMediaFrameworkCapture::GetPanelInstance();
	if (Panel.IsValid())
	{
		Panel->EnabledCapture(true);
	}
}

void UMediaFrameworkCapturePanel::StopCapture()
{
	TSharedPtr<SMediaFrameworkCapture> Panel = SMediaFrameworkCapture::GetPanelInstance();
	if (Panel.IsValid())
	{
		Panel->EnabledCapture(false);
	}
}

void UMediaFrameworkCapturePanel::EmptyRenderTargetCapture()
{
	TSharedPtr<SMediaFrameworkCapture> Panel = SMediaFrameworkCapture::GetPanelInstance();
	if (Panel.IsValid())
	{
		UMediaFrameworkWorldSettingsAssetUserData* UserData = Panel->FindOrAddMediaFrameworkAssetUserData();
		if (UserData)
		{
			UserData->RenderTargetCaptures.Empty();
		}
	}
}

void UMediaFrameworkCapturePanel::AddRenderTargetCapture(UMediaOutput* MediaOutput, UTextureRenderTarget2D* RenderTarget, FMediaCaptureOptions CaptureOptions)
{
	if (MediaOutput == nullptr || RenderTarget == nullptr)
	{
		UE_LOGF(LogMediaFrameworkUtilitiesEditor, Warning, "Invalid media output or render target.");
		return;
	}

	TSharedPtr<SMediaFrameworkCapture> Panel = SMediaFrameworkCapture::GetPanelInstance();
	if (Panel.IsValid())
	{
		UMediaFrameworkWorldSettingsAssetUserData* UserData = Panel->FindOrAddMediaFrameworkAssetUserData();
		if (UserData)
		{
			FMediaFrameworkCaptureRenderTargetCameraOutputInfo Data;
			Data.MediaOutput = MediaOutput;
			Data.RenderTarget = RenderTarget;
			Data.CaptureOptions = CaptureOptions;
			UserData->RenderTargetCaptures.Add(Data);
		}
	}
}

void UMediaFrameworkCapturePanel::EmptyViewportCapture()
{
	TSharedPtr<SMediaFrameworkCapture> Panel = SMediaFrameworkCapture::GetPanelInstance();
	if (Panel.IsValid())
	{
		UMediaFrameworkWorldSettingsAssetUserData* UserData = Panel->FindOrAddMediaFrameworkAssetUserData();
		if (UserData)
		{
			UserData->ViewportCaptures.Empty();
		}
	}
}

void UMediaFrameworkCapturePanel::AddViewportCapture(UMediaOutput* MediaOutput, AActor* Camera, FMediaCaptureOptions CaptureOptions, EViewModeIndex ViewMode)
{
	if (MediaOutput == nullptr || Camera == nullptr)
	{
		UE_LOGF(LogMediaFrameworkUtilitiesEditor, Warning, "Invalid media output or camera actor.");
		return;
	}

	TSharedPtr<SMediaFrameworkCapture> Panel = SMediaFrameworkCapture::GetPanelInstance();
	if (Panel.IsValid())
	{
		UMediaFrameworkWorldSettingsAssetUserData* UserData = Panel->FindOrAddMediaFrameworkAssetUserData();
		if (UserData)
		{
			FMediaFrameworkCaptureCameraViewportCameraOutputInfo Data;
			Data.MediaOutput = MediaOutput;
			Data.Cameras.Add(Camera);
			Data.CaptureOptions = CaptureOptions;
			Data.ViewMode = ViewMode;
			UserData->ViewportCaptures.Emplace(MoveTemp(Data));
		}
	}
}


void UMediaFrameworkCapturePanel::SetCurrentViewportCapture(UMediaOutput* MediaOutput, FMediaCaptureOptions CaptureOptions, EViewModeIndex ViewMode)
{
	TSharedPtr<SMediaFrameworkCapture> Panel = SMediaFrameworkCapture::GetPanelInstance();
	if (Panel.IsValid())
	{
		UMediaFrameworkWorldSettingsAssetUserData* UserData = Panel->FindOrAddMediaFrameworkAssetUserData();
		if (UserData)
		{
			UserData->CurrentViewportMediaOutput.MediaOutput = MediaOutput;
			UserData->CurrentViewportMediaOutput.CaptureOptions = CaptureOptions;
			UserData->CurrentViewportMediaOutput.ViewMode = ViewMode;
		}
	}
}


UMediaFrameworkCapturePanel* UMediaFrameworkCapturePanelBlueprintLibrary::GetMediaCapturePanel()
{
	return GetMutableDefault<UMediaFrameworkCapturePanel>();
}

#undef LOCTEXT_NAMESPACE
