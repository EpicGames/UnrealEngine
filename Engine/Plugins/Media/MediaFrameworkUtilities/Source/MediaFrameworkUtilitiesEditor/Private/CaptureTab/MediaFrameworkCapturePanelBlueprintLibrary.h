// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Engine/EngineBaseTypes.h"

#include "MediaFrameworkCapturePanelBlueprintLibrary.generated.h"

struct FMediaCaptureOptions;


class AActor;
class UMediaFrameworkWorldSettingsAssetUserData;
class UMediaOutput;
class UTextureRenderTarget2D;


UCLASS(MinimalAPI)
class UMediaFrameworkCapturePanel : public UObject
{
public:
	GENERATED_BODY()

	/**
	 * Capture the camera's viewport and the render target.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Media Capture")
	void StartCapture();

	/**
	 * Stop the current capture.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Media Capture")
	void StopCapture();

	/**
	 * Clear all the render target captures.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Media Capture")
	void EmptyRenderTargetCapture();

	/**
	 * Add a render target 2d to be captured.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Media Capture")
	void AddRenderTargetCapture(UMediaOutput* MediaOutput, UTextureRenderTarget2D* RenderTarget, FMediaCaptureOptions CaptureOptions);

	/**
	 * Clear all the viewport captures.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Media Capture")
	void EmptyViewportCapture();

	/**
	 * Add a camera to be used when capturing the current viewport.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Media Capture")
	void AddViewportCapture(UMediaOutput* MediaOutput, AActor* Camera, FMediaCaptureOptions CaptureOptions, EViewModeIndex ViewMode = VMI_Unknown);

	/**
	 * Change the setting for capturing the current viewport.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Media Capture")
	void SetCurrentViewportCapture(UMediaOutput* MediaOutput, FMediaCaptureOptions CaptureOptions, EViewModeIndex ViewMode = VMI_Unknown);
};


UCLASS(MinimalAPI, meta=(ScriptName="MediaFrameworkCapturePanelLibrary"))
class UMediaFrameworkCapturePanelBlueprintLibrary : public UBlueprintFunctionLibrary
{
public:
	GENERATED_BODY()

	/**
	 * Get Media Capture panel instance.
	 */
	UFUNCTION(BlueprintCallable, Category="Editor Scripting | Media Capture")
	static UMediaFrameworkCapturePanel* GetMediaCapturePanel();
};
