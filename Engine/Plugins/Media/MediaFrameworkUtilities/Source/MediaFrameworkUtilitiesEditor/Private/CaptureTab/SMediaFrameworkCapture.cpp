// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureTab/SMediaFrameworkCapture.h"

#include "DetailsViewArgs.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "LevelEditor.h"
#include "Materials/Material.h" // IWYU pragma: keep
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "UI/MediaFrameworkUtilitiesEditorStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"

#include "CaptureTab/SMediaFrameworkCaptureOutputWidget.h"
#include "GameFramework/WorldSettings.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "MediaFrameworkUtilities"

TWeakPtr<SMediaFrameworkCapture> SMediaFrameworkCapture::WidgetInstance;

namespace MediaFrameworkUtilities
{
	static const FName MediaFrameworkUtilitiesApp = FName("MediaFrameworkCaptureCameraViewportApp");
	static const FName LevelEditorModuleName("LevelEditor");

	TSharedRef<SDockTab> CreateMediaFrameworkCaptureCameraViewportTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SMediaFrameworkCapture)
			];
	}

	static float CaptureVerticalBoxPadding = 4.f;
	class SCaptureVerticalBox : public SVerticalBox
	{
	public:
		TWeakPtr<SMediaFrameworkCapture> Owner;
		TArray<TSharedPtr<SMediaFrameworkCaptureOutputWidget>> CaptureOutputWidget;

		void AddCaptureWidget(const TSharedPtr<SMediaFrameworkCaptureOutputWidget>& InWidget)
		{
			AddSlot()
			.Padding(FMargin(0.0f, CaptureVerticalBoxPadding, 0.0f, 0.0f))
			[
				InWidget.ToSharedRef()
			];

			CaptureOutputWidget.Add(InWidget);
		}

		void RemoveCaptureWidget(const TSharedPtr<SMediaFrameworkCaptureOutputWidget>& InWidget)
		{
			RemoveSlot(InWidget.ToSharedRef());
			CaptureOutputWidget.RemoveSingleSwap(InWidget);
		}

		virtual FVector2D ComputeDesiredSize(float Scale) const override
		{
			FVector2D SuperComputeDesiredSize = SVerticalBox::ComputeDesiredSize(Scale);
			float ChildComputeDesiredSizeY = 0.f;
			for(TSharedPtr<SMediaFrameworkCaptureOutputWidget> Widget : CaptureOutputWidget)
			{
				const FVector2D& CurChildDesiredSize = Widget->GetDesiredSize();
				ChildComputeDesiredSizeY += CurChildDesiredSize.Y;
				ChildComputeDesiredSizeY += CaptureVerticalBoxPadding;
			}
			return FVector2D(SuperComputeDesiredSize.X, FMath::Max(SuperComputeDesiredSize.Y, ChildComputeDesiredSizeY));
		}

		void OnPrePIE()
		{
			for (const auto& OutputWidget : CaptureOutputWidget)
			{
				if (OutputWidget.IsValid())
				{
					OutputWidget->OnPrePIE();
				}
			}
		}

		void OnPostPIEStarted()
		{
			for (const auto& OutputWidget : CaptureOutputWidget)
			{
				if (OutputWidget.IsValid())
				{
					OutputWidget->OnPostPIEStarted();
				}
			}
		}

		void OnPrePIEEnded()
		{
			for (const auto& OutputWidget : CaptureOutputWidget)
			{
				if (OutputWidget.IsValid())
				{
					OutputWidget->OnPrePIEEnded();
				}
			}
		}
	};
}

FDelegateHandle SMediaFrameworkCapture::LevelEditorTabManagerChangedHandle;

void SMediaFrameworkCapture::RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem)
{
	auto RegisterTabSpawner = [InWorkspaceItem]()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(MediaFrameworkUtilities::LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		LevelEditorTabManager->RegisterTabSpawner(MediaFrameworkUtilities::MediaFrameworkUtilitiesApp, FOnSpawnTab::CreateStatic(&MediaFrameworkUtilities::CreateMediaFrameworkCaptureCameraViewportTab))
			.SetDisplayName(LOCTEXT("TabTitle", "Media Capture"))
			.SetTooltipText(LOCTEXT("TooltipText", "Displays Capture Camera Viewport and Render Target."))
			.SetGroup(InWorkspaceItem)
			.SetIcon(FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), "TabIcons.MediaCapture.Small"));
	};

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	if (LevelEditorModule.GetLevelEditorTabManager())
	{
		RegisterTabSpawner();
	}
	else
	{
		LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterTabSpawner);
	}
}

void SMediaFrameworkCapture::UnregisterNomadTabSpawner()
{
	if (FSlateApplication::IsInitialized() && FModuleManager::Get().IsModuleLoaded(MediaFrameworkUtilities::LevelEditorModuleName))
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(MediaFrameworkUtilities::LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager;
		if (LevelEditorModule)
		{
			LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager();
			LevelEditorModule->OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
		}

		if (LevelEditorTabManager.IsValid())
		{
			LevelEditorTabManager->UnregisterTabSpawner(MediaFrameworkUtilities::MediaFrameworkUtilitiesApp);
		}
	}
}

TSharedPtr<SMediaFrameworkCapture> SMediaFrameworkCapture::GetPanelInstance()
{
	return SMediaFrameworkCapture::WidgetInstance.Pin();
}

SMediaFrameworkCapture::~SMediaFrameworkCapture()
{
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.RemoveAll(this);
	FEditorDelegates::OnAssetsDeleted.RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::PrePIEEnded.RemoveAll(this);
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
	FEditorDelegates::PreBeginPIE.RemoveAll(this);
	EnabledCapture(false);
	
	for (const TWeakObjectPtr<UMediaOutput>& WeakMediaOutput : MediaOutputsWithDelegates)
	{
		if (UMediaOutput* MediaOutput = WeakMediaOutput.Get())
		{
			MediaOutput->OnOutputModified().RemoveAll(this);
		}
	}
	MediaOutputsWithDelegates.Reset();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMediaFrameworkCapture::Construct(const FArguments& InArgs)
{
	WidgetInstance = StaticCastSharedRef<SMediaFrameworkCapture>(AsShared());

	bIsCapturing = false;
	bIsInPIESession = false;

	if (GEditor != nullptr)
	{
		bIsInPIESession = GEditor->PlayWorld != nullptr || GIsPlayInEditorWorld;
	}

	FEditorDelegates::PreBeginPIE.AddSP(this, &SMediaFrameworkCapture::OnPrePIE);
	FEditorDelegates::PostPIEStarted.AddSP(this, &SMediaFrameworkCapture::OnPostPIEStarted);
	FEditorDelegates::PrePIEEnded.AddSP(this, &SMediaFrameworkCapture::OnPrePIEEnded);
	FEditorDelegates::MapChange.AddSP(this, &SMediaFrameworkCapture::OnMapChange);
	GEngine->OnLevelActorDeleted().AddSP(this, &SMediaFrameworkCapture::OnLevelActorsRemoved);
	FEditorDelegates::OnAssetsDeleted.AddSP(this, &SMediaFrameworkCapture::OnAssetsDeleted);
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddRaw(this, &SMediaFrameworkCapture::OnObjectPreEditChange);

	UMediaFrameworkWorldSettingsAssetUserData* AssetUserData = FindOrAddMediaFrameworkAssetUserData();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = "MediaFrameworkUtilitites";
	DetailView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SMediaFrameworkCapture::IsPropertyReadOnly));
	DetailView->SetObject(AssetUserData);

	SAssignNew(CaptureBoxes, MediaFrameworkUtilities::SCaptureVerticalBox);
	CaptureBoxes->Owner = StaticCastSharedRef<SMediaFrameworkCapture>(AsShared());

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(2.f))
		[
			MakeToolBar()
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(FMargin(2.f))
		[
			SAssignNew(Splitter, SSplitter)
			.Orientation(GetDefault<UMediaFrameworkMediaCaptureSettings>()->bIsVerticalSplitterOrientation ? EOrientation::Orient_Vertical : EOrientation::Orient_Horizontal)
			+ SSplitter::Slot()
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.IsEnabled_Lambda([this]() { return !IsCapturing(); })
				[
					DetailView.ToSharedRef()
				]
			]
			+ SSplitter::Slot()
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						CaptureBoxes.ToSharedRef()
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<class SWidget> SMediaFrameworkCapture::MakeToolBar()
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
	ToolBarBuilder.BeginSection(TEXT("Player"));
	{
		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					EnabledCapture(true);
				}),
				FCanExecuteAction::CreateLambda([this]
				{
					return CanEnableViewport() && !IsCapturing();
				})),
			NAME_None,
			LOCTEXT("Output_Label", "Capture"),
			LOCTEXT("Output_ToolTip", "Capture the camera's viewport and the render target."),
			FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), "MediaCapture.Capture")
			);
		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					EnabledCapture(false);
				}),
				FCanExecuteAction::CreateLambda([this]
				{
					return IsCapturing();
				})
			),
			NAME_None,
			LOCTEXT("Stop_Label", "Stop"),
			LOCTEXT("Stop_ToolTip", "Stop the capturing of the camera's viewport and the render target."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop")
			);
	}
	ToolBarBuilder.EndSection();

	FSlimHorizontalToolBarBuilder RightToolBarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);

	RightToolBarBuilder.BeginSection("Options");
	{
		FUIAction OpenSettingsMenuAction;
		OpenSettingsMenuAction.CanExecuteAction = FCanExecuteAction::CreateLambda([this] { return !IsCapturing(); });

		RightToolBarBuilder.AddComboButton(
			OpenSettingsMenuAction,
			FOnGetContent::CreateRaw(this, &SMediaFrameworkCapture::CreateSettingsMenu),
			LOCTEXT("Settings_Label", "Settings"),
			LOCTEXT("Settings_ToolTip", "Settings"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Settings")
		);
	}
	RightToolBarBuilder.EndSection();


	return SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	[
		ToolBarBuilder.MakeWidget()
	]

	+SHorizontalBox::Slot()
	.AutoWidth()
	[
		RightToolBarBuilder.MakeWidget()
	];
}

TSharedRef<SWidget> SMediaFrameworkCapture::CreateSettingsMenu()
{
	FMenuBuilder SettingsMenuBuilder(true, nullptr);

	{
		SettingsMenuBuilder.AddMenuEntry(
			LOCTEXT("SplitterOrientation_Label", "Vertical Split"),
			LOCTEXT("SplitterOrientation_Tooltip", "Split the captures vertically or horizontally."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					const FScopedTransaction Transaction(LOCTEXT("CaptureSplitterOrientation", "Capture Splitter Orientation"));
					GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->Modify();
					GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->bIsVerticalSplitterOrientation = !GetDefault<UMediaFrameworkMediaCaptureSettings>()->bIsVerticalSplitterOrientation;
					Splitter->SetOrientation(GetDefault<UMediaFrameworkMediaCaptureSettings>()->bIsVerticalSplitterOrientation ? EOrientation::Orient_Vertical : EOrientation::Orient_Horizontal);
					GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]
				{
					return GetDefault<UMediaFrameworkMediaCaptureSettings>()->bIsVerticalSplitterOrientation;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);

		SettingsMenuBuilder.AddMenuEntry(
			LOCTEXT("SaveCaptureSetingsInWorld_Label", "Capture Settings Per Level"),
			LOCTEXT("SaveCaptureSetingsInWorld_Tooltip", "Should the Capture Setting be saved with the level or should they be saved in the editor settings (and be the same in every level)."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					const FScopedTransaction Transaction(LOCTEXT("CaptureSettingPerLevel", "Capture Settings Per World"));
					GetMutableDefault<UMediaFrameworkEditorCaptureSettings>()->Modify();
					GetMutableDefault<UMediaFrameworkEditorCaptureSettings>()->bSaveCaptureSetingsInWorld = !GetDefault<UMediaFrameworkEditorCaptureSettings>()->bSaveCaptureSetingsInWorld;
					DetailView->SetObject(FindOrAddMediaFrameworkAssetUserData());
					GetMutableDefault<UMediaFrameworkEditorCaptureSettings>()->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]
				{
					return GetDefault<UMediaFrameworkEditorCaptureSettings>()->bSaveCaptureSetingsInWorld;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);

		SettingsMenuBuilder.AddMenuEntry(
			LOCTEXT("AutoRestartCaptureOnChange_Label", "Auto Restart On Change"),
			LOCTEXT("AutoRestartCaptureOnChange_Tooltip", "Should the capture be restarted if the media output is modified."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]
					{
						const FScopedTransaction Transaction(LOCTEXT("AutoRestartCaptureOnChange", "Auto Restart Capture On Change"));
						GetMutableDefault<UMediaFrameworkEditorCaptureSettings>()->Modify();
						GetMutableDefault<UMediaFrameworkEditorCaptureSettings>()->bAutoRestartCaptureOnChange = !GetDefault<UMediaFrameworkEditorCaptureSettings>()->bAutoRestartCaptureOnChange;
						DetailView->SetObject(FindOrAddMediaFrameworkAssetUserData());
						GetMutableDefault<UMediaFrameworkEditorCaptureSettings>()->SaveConfig();
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]
					{
						return GetDefault<UMediaFrameworkEditorCaptureSettings>()->bAutoRestartCaptureOnChange;
					})
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);

		SettingsMenuBuilder.BeginSection("ViewportCaptureSection", LOCTEXT("ViewportCaptureSectionHeader", "Viewport Capture"));
		{
			typedef SNumericVectorInputBox<int32, FIntPoint, 2> SNumericVectorInputBoxInt2;

			const TSharedRef<SWidget> InputBox = SNew(SNumericVectorInputBoxInt2)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.AllowSpin(false)
				.bColorAxisLabels(false)
				.X(this, &SMediaFrameworkCapture::GetDefaultViewportSize, 0)
				.OnXChanged(this, &SMediaFrameworkCapture::DefaultViewportSizeChanged, 0)
				.OnXCommitted(this, &SMediaFrameworkCapture::DefaultViewportSizeCommitted, 0)
				.Y(this, &SMediaFrameworkCapture::GetDefaultViewportSize, 1)
				.OnYChanged(this, &SMediaFrameworkCapture::DefaultViewportSizeChanged, 1)
				.OnYCommitted(this, &SMediaFrameworkCapture::DefaultViewportSizeCommitted, 1);
			
			SettingsMenuBuilder.AddWidget(InputBox,
				LOCTEXT("DefaultViewportCaptureSize_Label", "Default Size"),
				false /*bInNoIndent*/,
				true /*bInSearchable*/,
				LOCTEXT("DefaultViewportCaptureSize_Tooltip", "Default resolution to use for viewport capture if it is not specified by the media output.")
				);
		}

		SettingsMenuBuilder.EndSection();
	}

	return SettingsMenuBuilder.MakeWidget();
}

TOptional<int32> SMediaFrameworkCapture::GetDefaultViewportSize(int32 InValueIndex) const
{
	if (InValueIndex >= 0 && InValueIndex < 2)
	{
		return GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->DefaultViewportCaptureSize(InValueIndex);
	}
	return TOptional<int32>();
}
	
void SMediaFrameworkCapture::DefaultViewportSizeChanged(int32 InValue, int32 InValueIndex)
{
	if (InValueIndex >= 0 && InValueIndex < 2)
	{
		GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->DefaultViewportCaptureSize(InValueIndex) = InValue;
	}
}
	
void SMediaFrameworkCapture::DefaultViewportSizeCommitted(int32 InValue, ETextCommit::Type InCommitInfo, int32 InValueIndex)
{
	if (InCommitInfo == ETextCommit::OnEnter || InCommitInfo == ETextCommit::OnUserMovedFocus)
	{
		if (InValueIndex >= 0 && InValueIndex < 2)
		{
			const FScopedTransaction Transaction(LOCTEXT("DefaultViewportCaptureSize_Transaction", "Default Viewport Capture Size"));
			GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->Modify();
			GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->DefaultViewportCaptureSize(InValueIndex) = InValue;
			GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->SaveConfig();
		}
	}
}

bool SMediaFrameworkCapture::CanEnableViewport() const
{
	UMediaFrameworkWorldSettingsAssetUserData* AssetUserData = FindMediaFrameworkAssetUserData();
	bool bEnabled = AssetUserData && (AssetUserData->ViewportCaptures.Num() || AssetUserData->RenderTargetCaptures.Num() || AssetUserData->CurrentViewportMediaOutput.MediaOutput != nullptr);
	if (bEnabled)
	{
		for (const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& Info : AssetUserData->ViewportCaptures)
		{
			bEnabled = Info.MediaOutput && Info.Cameras.Num() > 0;
			for (const TSoftObjectPtr<AActor>& CameraActorRef : Info.Cameras)
			{
				bEnabled = bEnabled && CameraActorRef.IsValid();
			}

			if (!bEnabled)
			{
				break;
			}
		}

		if (bEnabled)
		{
			for (const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& Info : AssetUserData->RenderTargetCaptures)
			{
				bEnabled = Info.MediaOutput && Info.RenderTarget;

				if (!bEnabled)
				{
					break;
				}
			}
		}
	}
	return bEnabled;
}

void SMediaFrameworkCapture::EnabledCapture(bool bEnabled)
{
	if (bEnabled)
	{
		bEnabled = CanEnableViewport();
	}

	if (bEnabled)
	{
		if (bIsCapturing)
		{
			EnabledCapture(false);
			for (const TWeakObjectPtr<UMediaOutput>& WeakMediaOutput : MediaOutputsWithDelegates)
			{
				if (UMediaOutput* MediaOutput = WeakMediaOutput.Get())
				{
					MediaOutput->OnOutputModified().RemoveAll(this);
				}
			}
			
			MediaOutputsWithDelegates.Reset();
		}

		UMediaFrameworkWorldSettingsAssetUserData* AssetUserData = FindOrAddMediaFrameworkAssetUserData();
		check(AssetUserData);
		for (const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& Info : AssetUserData->ViewportCaptures)
		{
			TArray<TWeakObjectPtr<AActor>> InfoPreviewActors;
			InfoPreviewActors.Reserve(Info.Cameras.Num());
			for (const TSoftObjectPtr<AActor>& ActorRef : Info.Cameras)
			{
				AActor* Actor = ActorRef.Get();
				if (Actor)
				{
					InfoPreviewActors.Add(Actor);
				}
			}

			TSharedPtr<SMediaFrameworkCaptureCameraViewportWidget> CaptureCameraViewport = SNew(SMediaFrameworkCaptureCameraViewportWidget)
				.Owner(SharedThis(this))
				.PreviewActors(InfoPreviewActors)
				.MediaOutput(Info.MediaOutput)
				.CaptureOptions(Info.CaptureOptions)
				.ViewMode(Info.ViewMode);

			Info.MediaOutput->OnOutputModified().AddSP(this, &SMediaFrameworkCapture::OnOutputModified);
			MediaOutputsWithDelegates.Add(Info.MediaOutput);

			CaptureBoxes->AddCaptureWidget(CaptureCameraViewport);
			CaptureCameraViewport->StartOutput();
			CaptureCameraViewports.Add(CaptureCameraViewport);
		}

		for (const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& Info : AssetUserData->RenderTargetCaptures)
		{
			TSharedPtr<SMediaFrameworkCaptureRenderTargetWidget> CaptureRenderTarget = SNew(SMediaFrameworkCaptureRenderTargetWidget)
				.Owner(SharedThis(this))
				.MediaOutput(Info.MediaOutput)
				.CaptureOptions(Info.CaptureOptions)
				.RenderTarget(Info.RenderTarget);

			Info.MediaOutput->OnOutputModified().AddSP(this, &SMediaFrameworkCapture::OnOutputModified);
			MediaOutputsWithDelegates.Add(Info.MediaOutput);

			CaptureBoxes->AddCaptureWidget(CaptureRenderTarget);
			CaptureRenderTarget->StartOutput();
			CaptureRenderTargets.Add(CaptureRenderTarget);
		}

		if (AssetUserData->CurrentViewportMediaOutput.MediaOutput != nullptr)
		{
			SAssignNew(CaptureCurrentViewport, SMediaFrameworkCaptureCurrentViewportWidget)
				.Owner(SharedThis(this))
				.MediaOutput(AssetUserData->CurrentViewportMediaOutput.MediaOutput)
				.CaptureOptions(AssetUserData->CurrentViewportMediaOutput.CaptureOptions)
				.ViewMode(AssetUserData->CurrentViewportMediaOutput.ViewMode);

			AssetUserData->CurrentViewportMediaOutput.MediaOutput->OnOutputModified().AddSP(this, &SMediaFrameworkCapture::OnOutputModified);
			MediaOutputsWithDelegates.Add(AssetUserData->CurrentViewportMediaOutput.MediaOutput);

			CaptureBoxes->AddCaptureWidget(CaptureCurrentViewport);
			CaptureCurrentViewport->StartOutput();
		}
	}
	else
	{
		for (TSharedPtr<SMediaFrameworkCaptureCameraViewportWidget> CaptureCameraViewport : CaptureCameraViewports)
		{
			CaptureBoxes->RemoveCaptureWidget(CaptureCameraViewport.ToSharedRef());
		}
		CaptureCameraViewports.Reset();

		for (TSharedPtr<SMediaFrameworkCaptureRenderTargetWidget> CaptureRenderTarget : CaptureRenderTargets)
		{
			CaptureBoxes->RemoveCaptureWidget(CaptureRenderTarget.ToSharedRef());
		}
		CaptureRenderTargets.Reset();

		if (CaptureCurrentViewport.IsValid())
		{
			CaptureBoxes->RemoveCaptureWidget(CaptureCurrentViewport);
		}
		CaptureCurrentViewport.Reset();
	}

	bIsCapturing = CaptureCameraViewports.Num() > 0 || CaptureRenderTargets.Num() > 0 || CaptureCurrentViewport.IsValid();
}

UMediaFrameworkWorldSettingsAssetUserData* SMediaFrameworkCapture::FindMediaFrameworkAssetUserData() const
{
	UMediaFrameworkWorldSettingsAssetUserData* Result = nullptr;

	if (GetDefault<UMediaFrameworkEditorCaptureSettings>()->bSaveCaptureSetingsInWorld)
	{
		UWorld* World = GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
		if (World)
		{
			AWorldSettings* WorldSetting = World ? World->GetWorldSettings() : nullptr;
			if (WorldSetting)
			{
				Result = CastChecked<UMediaFrameworkWorldSettingsAssetUserData>(WorldSetting->GetAssetUserDataOfClass(UMediaFrameworkWorldSettingsAssetUserData::StaticClass()), ECastCheckedType::NullAllowed);
			}
		}
	}
	else
	{
		Result = GetMutableDefault<UMediaFrameworkEditorCaptureSettings>();
	}

	return Result;
}

UMediaFrameworkWorldSettingsAssetUserData* SMediaFrameworkCapture::FindOrAddMediaFrameworkAssetUserData()
{
	UMediaFrameworkWorldSettingsAssetUserData* Result = nullptr;

	if (GetDefault<UMediaFrameworkEditorCaptureSettings>()->bSaveCaptureSetingsInWorld)
	{
		UWorld* World = GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
		if (World)
		{
			AWorldSettings* WorldSetting = World ? World->GetWorldSettings() : nullptr;
			if (WorldSetting)
			{
				Result = CastChecked<UMediaFrameworkWorldSettingsAssetUserData>(WorldSetting->GetAssetUserDataOfClass(UMediaFrameworkWorldSettingsAssetUserData::StaticClass()), ECastCheckedType::NullAllowed);

				if (!Result)
				{
					Result = NewObject<UMediaFrameworkWorldSettingsAssetUserData>(WorldSetting);
					WorldSetting->AddAssetUserData(Result);
				}
			}
		}
	}
	else
	{
		Result = GetMutableDefault<UMediaFrameworkEditorCaptureSettings>();
	}

	return Result;
}

void SMediaFrameworkCapture::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (GetDefault<UMediaFrameworkEditorCaptureSettings>()->bSaveCaptureSetingsInWorld)
	{
		GetMutableDefault<UMediaFrameworkEditorCaptureSettings>()->SaveConfig();
	}
}

bool SMediaFrameworkCapture::IsPropertyReadOnly(const FPropertyAndParent& PropertyAndParent) const
{
	return !GetDefault<UMediaFrameworkEditorCaptureSettings>()->bSaveCaptureSetingsInWorld && PropertyAndParent.Property.GetFName() == GET_MEMBER_NAME_CHECKED(UMediaFrameworkWorldSettingsAssetUserData, ViewportCaptures);
}

void SMediaFrameworkCapture::OnMapChange(uint32 MapFlags)
{
	UMediaFrameworkWorldSettingsAssetUserData* AssetUserData = FindOrAddMediaFrameworkAssetUserData();
	DetailView->SetObject(AssetUserData);
	EnabledCapture(false);
}

void SMediaFrameworkCapture::OnLevelActorsRemoved(AActor* InActor)
{
	UMediaFrameworkWorldSettingsAssetUserData* AssetUserData = FindMediaFrameworkAssetUserData();
	if (AssetUserData)
	{
		for (const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& Info : AssetUserData->ViewportCaptures)
		{
			if (Info.Cameras.Contains(InActor))
			{
				EnabledCapture(false);
				return;
			}
		}
	}
}

void SMediaFrameworkCapture::OnAssetsDeleted(const TArray<UClass*>& DeletedAssetClasses)
{
	if (bIsCapturing)
	{
		bool bCheck = false;
		for(UClass* AssetClass : DeletedAssetClasses)
		{
			if (AssetClass->IsChildOf<UTextureRenderTarget2D>())
			{
				bCheck = true;
				break;
			}
		}

		if (bCheck)
		{
			for(const TSharedPtr<SMediaFrameworkCaptureRenderTargetWidget>& Capture : CaptureRenderTargets)
			{
				if (!Capture->IsValid())
				{
					EnabledCapture(false);
					return;
				}
			}
		}
	}
}

void SMediaFrameworkCapture::OnObjectPreEditChange(UObject* Object, const FEditPropertyChain& PropertyChain)
{
	UMediaFrameworkWorldSettingsAssetUserData* AssetUserData = FindMediaFrameworkAssetUserData();
	if (Object == AssetUserData)
	{
		EnabledCapture(false);
	}
}

void SMediaFrameworkCapture::OnOutputModified(UMediaOutput* Output)
{
	if (bIsCapturing)
	{
		if (GetDefault<UMediaFrameworkEditorCaptureSettings>()->bAutoRestartCaptureOnChange)
		{
			/** Only trigger one restart if multiple properties were modified. */
			if (!NextTickTimerHandle.IsValid())
			{
				for (const TSharedPtr<SMediaFrameworkCaptureCameraViewportWidget>& CaptureWidget : CaptureCameraViewports)
				{
					CaptureWidget->StopOutput();
				}
				
				for (const TSharedPtr<SMediaFrameworkCaptureRenderTargetWidget>& CaptureWidget : CaptureRenderTargets)
				{
					CaptureWidget->StopOutput();
				}
				
				if (CaptureCurrentViewport)
				{
					CaptureCurrentViewport->StopOutput();
				}

				NextTickTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda(
					[]() mutable
				{
					if (TSharedPtr<SMediaFrameworkCapture> Capture = WidgetInstance.Pin())
					{
						Capture->EnabledCapture(true);
						Capture->NextTickTimerHandle.Invalidate();
					}
				}));
			}
		}
	}
}

void SMediaFrameworkCapture::OnPrePIE(bool)
{
	if (CaptureBoxes.IsValid())
	{
		CaptureBoxes->OnPrePIE();
	}
}

void SMediaFrameworkCapture::OnPostPIEStarted(bool)
{
	bIsInPIESession = true;

	if (CaptureBoxes.IsValid())
	{
		CaptureBoxes->OnPostPIEStarted();
	}
}

void SMediaFrameworkCapture::OnPrePIEEnded(bool)
{
	bIsInPIESession = false;

	if (CaptureBoxes.IsValid())
	{
		CaptureBoxes->OnPrePIEEnded();
	}
}


#undef LOCTEXT_NAMESPACE
