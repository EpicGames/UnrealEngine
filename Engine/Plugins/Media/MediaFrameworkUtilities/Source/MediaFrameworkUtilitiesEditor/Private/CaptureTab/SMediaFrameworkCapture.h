// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Engine/TimerHandle.h"
#include "MediaFrameworkWorldSettingsAssetUserData.h"
#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

#include "SMediaFrameworkCapture.generated.h"

struct FPropertyAndParent;

class FWorkspaceItem;
class IDetailsView;
class SMediaFrameworkCaptureCameraViewportWidget;
class SMediaFrameworkCaptureCurrentViewportWidget;
class SMediaFrameworkCaptureRenderTargetWidget;
class SSplitter;

namespace MediaFrameworkUtilities
{
	class SCaptureVerticalBox;
}

/**
 * Settings for the media capture tab.
 */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UMediaFrameworkMediaCaptureSettings : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(config)
	bool bIsVerticalSplitterOrientation = true;
	
	/**
	 * This is the resolution to use if it is not specified by the media output.
	 */
	UPROPERTY(EditAnywhere, config, Category="Media Viewport Capture")
	FIntPoint DefaultViewportCaptureSize = FIntPoint(1920, 1080);
};

/**
 * Settings for the capture that are persistent per users.
 */
UCLASS(MinimalAPI, config = Editor)
class UMediaFrameworkEditorCaptureSettings : public UMediaFrameworkWorldSettingsAssetUserData
{
	GENERATED_BODY()
public:
	/** Should the Capture setting be saved with the level or should it be saved as a project settings. */
	UPROPERTY(config)
	bool bSaveCaptureSetingsInWorld = true;
	
	/** Should the capture be restarted if the media output is modified. */
	UPROPERTY(config)
	bool bAutoRestartCaptureOnChange = true;
};

/*
 * SMediaFrameworkCapture
 */
class SMediaFrameworkCapture : public SCompoundWidget, public FNotifyHook
{
public:
	static void RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem);
	static void UnregisterNomadTabSpawner();
	static TSharedPtr<SMediaFrameworkCapture> GetPanelInstance();

private:
	static TWeakPtr<SMediaFrameworkCapture> WidgetInstance;

public:
	SLATE_BEGIN_ARGS(SMediaFrameworkCapture){}
	SLATE_END_ARGS()

	~SMediaFrameworkCapture();

	void Construct(const FArguments& InArgs);

	bool IsCapturing() const { return bIsCapturing; }
	void EnabledCapture(bool bEnabled);
	UMediaFrameworkWorldSettingsAssetUserData* FindOrAddMediaFrameworkAssetUserData();

private:
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	bool IsPropertyReadOnly(const FPropertyAndParent& PropertyAndParent) const;

	void OnMapChange(uint32 InMapFlags);
	void OnLevelActorsRemoved(AActor* InActor);
	void OnAssetsDeleted(const TArray<UClass*>& DeletedAssetClasses);
	void OnObjectPreEditChange(UObject* Object, const FEditPropertyChain& PropertyChain);
	void OnOutputModified(UMediaOutput* Output);

	TSharedRef<class SWidget> MakeToolBar();
	TSharedRef<SWidget> CreateSettingsMenu();

	TOptional<int32> GetDefaultViewportSize(int32 InValueIndex) const;
	void DefaultViewportSizeChanged(int32 InValue, int32 InValueIndex);
	void DefaultViewportSizeCommitted(int32 InValue, ETextCommit::Type InCommitInfo, int32 InValueIndex);

	bool CanEnableViewport() const;
	UMediaFrameworkWorldSettingsAssetUserData* FindMediaFrameworkAssetUserData() const;

	void OnPrePIE(bool bIsSimulating);
	void OnPostPIEStarted(bool bIsSimulating);
	void OnPrePIEEnded(bool bIsSimulating);

	TSharedPtr<IDetailsView> DetailView;
	TSharedPtr<SSplitter> Splitter;
	TSharedPtr<MediaFrameworkUtilities::SCaptureVerticalBox> CaptureBoxes;
	bool bIsCapturing;
	bool bIsInPIESession;

	TArray<TSharedPtr<SMediaFrameworkCaptureCameraViewportWidget>> CaptureCameraViewports;
	TArray<TSharedPtr<SMediaFrameworkCaptureRenderTargetWidget>> CaptureRenderTargets;
	TSharedPtr<SMediaFrameworkCaptureCurrentViewportWidget> CaptureCurrentViewport;

	static FDelegateHandle LevelEditorTabManagerChangedHandle;

	/** Handle for the timer used to restart a source after its output options have been modified. */
	FTimerHandle NextTickTimerHandle;

	/** List of MediaOutputs for which we have a registered delegates. */
	TArray<TWeakObjectPtr<UMediaOutput>> MediaOutputsWithDelegates;
};
