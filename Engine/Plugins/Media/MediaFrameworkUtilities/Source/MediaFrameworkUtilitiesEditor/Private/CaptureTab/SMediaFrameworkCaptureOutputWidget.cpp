// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureTab/SMediaFrameworkCaptureOutputWidget.h"

#include "Editor/UnrealEdTypes.h"
#include "EditorFontGlyphs.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "LevelEditorViewport.h"
#include "SMediaFrameworkCapture.h"
#include "Slate/SceneViewport.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SMediaImage.h"
#include "Widgets/SViewport.h"
#include "Widgets/Text/STextBlock.h"

#include "Modules/ModuleManager.h"
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "MediaFrameworkUtilities"

/*
 * MediaFrameworkUtilities namespace
 */
namespace MediaFrameworkUtilities
{
	constexpr float Padding = 4.f;
	constexpr float ViewportBoxDesiredSizeY = 200.f;
	
	/**
	 * Determines the target size from the media output if desired size is specified, otherwise,
	 * fallback to the default viewport capture resolution.
	 */
	FIntPoint GetTargetSize(const TWeakObjectPtr<UMediaOutput>& InMediaOutputWeak)
	{
		if (const UMediaOutput* MediaOutputPtr = InMediaOutputWeak.Get())
		{
			const FIntPoint TargetSize = MediaOutputPtr->GetRequestedSize();
			if (TargetSize != UMediaOutput::RequestCaptureSourceSize)
			{
				return TargetSize;
			}
		}
		return GetMutableDefault<UMediaFrameworkMediaCaptureSettings>()->DefaultViewportCaptureSize;
	}
	
	float GetWidthOverride(const TWeakObjectPtr<UMediaOutput> InMediaOutputWeak)
	{
		float Ratio = 1.77777777f;
		const FIntPoint TargetSize = GetTargetSize(InMediaOutputWeak);
		if (TargetSize.X > 0 && TargetSize.Y > 0)
		{
			Ratio = (float)TargetSize.X / (float)TargetSize.Y;
		}
		return ViewportBoxDesiredSizeY * Ratio;
	}

	/*
	 * FMediaFrameworkCaptureLevelEditorViewportClient
	 * Like FLevelEditorViewportClient but always use the PlayWorld if available. Do not support bSimulated
	 */
	class FMediaFrameworkCaptureLevelEditorViewportClient : public FLevelEditorViewportClient
	{
	private:
		using Super = FLevelEditorViewportClient;

	public:
		FMediaFrameworkCaptureLevelEditorViewportClient(const TSharedPtr<class SLevelViewport>& InLevelViewport, EViewModeIndex InViewModeIndex)
			: FLevelEditorViewportClient(InLevelViewport)
			, ViewModeIndex(InViewModeIndex)
		{}
		~FMediaFrameworkCaptureLevelEditorViewportClient()
		{
			if (GEditor)
			{
				FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
				if (PIEWorldContext)
				{
					RemoveReferenceToWorldContext(*PIEWorldContext);
				}
			}
		}

		virtual UWorld* GetWorld() const override
		{
			if (GEditor->PlayWorld)
			{
				return GEditor->PlayWorld;
			}
			return Super::GetWorld();
		}

		void SetPIE(bool bInIsPIE)
		{
			FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
			if (bInIsPIE)
			{
				if (PIEWorldContext)
				{
					RemoveReferenceToWorldContext(GEditor->GetEditorWorldContext());
					SetReferenceToWorldContext(*PIEWorldContext);
				}
			}
			else
			{
				if (PIEWorldContext)
				{
					RemoveReferenceToWorldContext(*PIEWorldContext);
				}
				SetReferenceToWorldContext(GEditor->GetEditorWorldContext());
			}
			SetViewMode(ViewModeIndex);
			SetRealtime(true);
		}

		void AutoSetPIE()
		{
			if (GEditor->PlayWorld)
			{
				SetPIE(true);
			}
			else
			{
				SetViewMode(ViewModeIndex);
				SetRealtime(true);
			}
		}

	private:
		EViewModeIndex ViewModeIndex;
	};
}

/*
 * SMediaFrameworkCaptureOutputWidget implementation
 */
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMediaFrameworkCaptureOutputWidget::Construct(const FArguments& InArgs)
{
	Owner = InArgs._Owner;
	MediaOutput = InArgs._MediaOutput;
	CaptureOptions = InArgs._CaptureOptions;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SMediaFrameworkCaptureOutputWidget::StopOutput()
{
	if (MediaCapture.IsValid())
	{
		MediaCapture->StopCapture(false);
		MediaCapture.Reset();
	}
}

TSharedRef<SWidget> SMediaFrameworkCaptureOutputWidget::BuildBaseWidget(TSharedRef<SWidget> InnerWidget, const FString& CaptureType)
{
	const FMargin SourceTextPadding(6.f, 2.f, 0.f, 2.f);

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f, 0.0f)
		.FillHeight(1.0f)
		.VAlign(EVerticalAlignment::VAlign_Top)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(this, &SMediaFrameworkCaptureOutputWidget::HandleIconText)
				.ColorAndOpacity(this, &SMediaFrameworkCaptureOutputWidget::HandleIconColorAndOpacity)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.Padding(SourceTextPadding)
			[
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalText")))
				.Text(FText::FromString(CaptureType + TEXT(" - ") + MediaOutput->GetName()))
			]
		]
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.AutoHeight()
		[
			// Live view of the Source
			SNew(SBox)
			.Visibility(EVisibility::HitTestInvisible)
			.WidthOverride(MakeAttributeLambda([this]
			{
				return FOptionalSize(MediaFrameworkUtilities::GetWidthOverride(MediaOutput));
			}))
			.HeightOverride(MediaFrameworkUtilities::ViewportBoxDesiredSizeY)
			[
				InnerWidget
			]
		];
}

FSlateColor SMediaFrameworkCaptureOutputWidget::HandleIconColorAndOpacity() const
{
	FSlateColor Result = FLinearColor::Red;
	if (MediaCapture.IsValid())
	{
		EMediaCaptureState State = MediaCapture->GetState();
		switch (State)
		{
		case EMediaCaptureState::Error:
		case EMediaCaptureState::Stopped:
			Result = FLinearColor::Red;
			break;
		case EMediaCaptureState::Capturing:
			Result = FLinearColor::Green;
			break;
		case EMediaCaptureState::Preparing:
		case EMediaCaptureState::StopRequested:
			Result = FLinearColor::Yellow;
			break;
		}
	}
	return Result;
}

FText SMediaFrameworkCaptureOutputWidget::HandleIconText() const
{
	FText Result = FEditorFontGlyphs::Ban;
	if (MediaCapture.IsValid())
	{
		static FText Video_Slash = FText::FromString(FString(TEXT("\xf4e2")));

		EMediaCaptureState State = MediaCapture->GetState();
		switch (State)
		{
		case EMediaCaptureState::Error:
		case EMediaCaptureState::Stopped:
			Result = Video_Slash;
			break;
		case EMediaCaptureState::StopRequested:
			Result = FEditorFontGlyphs::Exclamation;
			break;
		case EMediaCaptureState::Capturing:
			Result = FEditorFontGlyphs::Video_Camera;
			break;
		case EMediaCaptureState::Preparing:
			Result = FEditorFontGlyphs::Hourglass_O;
			break;
		}
	}
	return Result;
}

bool SMediaFrameworkCaptureOutputWidget::IsValid() const
{
	return MediaOutput.IsValid() && MediaCapture.IsValid();
}

/*
 * SMediaFrameworkCaptureCameraViewportWidget implementation
 */
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMediaFrameworkCaptureCameraViewportWidget::Construct(const FArguments& InArgs)
{
	PreviewActors = InArgs._PreviewActors;
	ViewMode = InArgs._ViewMode;
	CurrentLockCameraIndex = 0;

	SMediaFrameworkCaptureOutputWidget::FArguments BaseArguments;
	BaseArguments._Owner = InArgs._Owner;
	BaseArguments._MediaOutput = InArgs._MediaOutput;
	BaseArguments._CaptureOptions = InArgs._CaptureOptions;
	SMediaFrameworkCaptureOutputWidget::Construct(BaseArguments);

	LevelViewportClient = MakeShareable(new MediaFrameworkUtilities::FMediaFrameworkCaptureLevelEditorViewportClient(TSharedPtr<class SLevelViewport>(), ViewMode));
	{
		// Preview viewports never be a listener
		LevelViewportClient->bSetListenerPosition = false;

		// Default to "game" show flags for camera previews
		LevelViewportClient->EngineShowFlags = FEngineShowFlags(ESFIM_Game);
		LevelViewportClient->LastEngineShowFlags = FEngineShowFlags(ESFIM_Editor);

		LevelViewportClient->ViewportType = LVT_Perspective;
		LevelViewportClient->bDrawAxes = false;
		LevelViewportClient->bDisableInput = true;
		LevelViewportClient->SetAllowCinematicControl(false);
		LevelViewportClient->VisibilityDelegate.BindLambda([] {return true; });
		LevelViewportClient->AutoSetPIE();
	}

	UpdateActivePreviewList(GEditor->PlayWorld != nullptr);
	LevelViewportClient->UpdateViewForLockedActor();

	ViewportWidget = SNew(SViewport)
		.RenderDirectlyToWindow(false)
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		.EnableGammaCorrection(false)		// Scene rendering handles gamma correction
		.EnableBlending(false);

	SceneViewport = FSceneViewport::Create(LevelViewportClient, ViewportWidget);


	TSharedPtr<SWidget> GridPanelWidget = SNullWidget::NullWidget;
	const int32 CameraNum = PreviewActors.Num();
	if (CameraNum > 1)
	{
		TSharedPtr<SGridPanel> GridPanel = SNew(SGridPanel);
		GridPanelWidget = GridPanel;

		const int32 MaxCameraPerRow = 5;

		int32 CameraIndexY = 0;
		int32 CameraCounter = 0;
		while (CameraCounter < CameraNum)
		{
			for (int32 IndexX = 0; IndexX < MaxCameraPerRow && CameraCounter < CameraNum; ++CameraCounter, ++IndexX)
			{
				GridPanel->AddSlot(IndexX, CameraIndexY)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(FText::FromString(PreviewActors[CameraCounter]->GetActorLabel()))
					.OnClicked(FOnClicked::CreateLambda([this, CameraCounter]()
					{
						SetActorLock(CameraCounter);
						return FReply::Handled();
					}))
					.IsEnabled_Lambda([this, CameraCounter]()
					{
						if (ActivePreviewActors.IsValidIndex(CameraCounter))
						{
							return ActivePreviewActors[CameraCounter].IsValid();
						}

						return false;
					})
				];
			}
			++CameraIndexY;
		}
	}

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(MediaFrameworkUtilities::Padding)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				GridPanelWidget.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoHeight()
			[
				BuildBaseWidget(ViewportWidget.ToSharedRef(), TEXT("Viewport Capture"))
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SMediaFrameworkCaptureCameraViewportWidget::~SMediaFrameworkCaptureCameraViewportWidget()
{
	StopOutput();
}

void SMediaFrameworkCaptureCameraViewportWidget::StartOutput()
{
	UMediaOutput* MediaOutputPtr = MediaOutput.Get();
	if (MediaOutputPtr && SceneViewport.IsValid())
	{
		MediaCapture.Reset(MediaOutputPtr->CreateMediaCapture());
		if (MediaCapture.IsValid())
		{
			const FIntPoint TargetSize = MediaFrameworkUtilities::GetTargetSize(MediaOutput);
			SceneViewport->SetFixedViewportSize(TargetSize.X, TargetSize.Y);
			MediaCapture->CaptureSceneViewport(SceneViewport, CaptureOptions);
		}
	}
}

void SMediaFrameworkCaptureCameraViewportWidget::StopOutput()
{
	Super::StopOutput();

	if (LevelViewportClient.IsValid())
	{
		LevelViewportClient.Reset();
		ViewportWidget.Reset();
		SceneViewport.Reset();
	}
}

void SMediaFrameworkCaptureCameraViewportWidget::OnPostPIEStarted()
{
	const bool bIsPIE = true;
	if (LevelViewportClient.IsValid())
	{
		LevelViewportClient->SetPIE(bIsPIE);
	}

	UpdateActivePreviewList(bIsPIE);
}

void SMediaFrameworkCaptureCameraViewportWidget::OnPrePIEEnded()
{
	const bool bIsPIE = false;
	UpdateActivePreviewList(bIsPIE);
	
	if (LevelViewportClient.IsValid())
	{
		LevelViewportClient->SetPIE(bIsPIE);
	}
}

void SMediaFrameworkCaptureCameraViewportWidget::SetActorLock(int32 CameraIndex)
{
	CurrentLockCameraIndex = CameraIndex;
	if (ActivePreviewActors.IsValidIndex(CurrentLockCameraIndex))
	{
		LevelViewportClient->SetActorLock(ActivePreviewActors[CurrentLockCameraIndex].Get());
	}
	else
	{
		LevelViewportClient->SetActorLock(nullptr);
	}
}

void SMediaFrameworkCaptureCameraViewportWidget::UpdateActivePreviewList(bool bIsPIE)
{
	if (bIsPIE)
	{
		ActivePreviewActors.Empty();
		for (TWeakObjectPtr<AActor> PreviewActor : PreviewActors)
		{
			AActor* PreviewActorPtr = PreviewActor.Get();
			if (PreviewActorPtr != nullptr)
			{
				const bool bIsAlreadyPIEActor = PreviewActorPtr->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
				if (!bIsAlreadyPIEActor)
				{
					AActor* CounterpartActor = EditorUtilities::GetSimWorldCounterpartActor(PreviewActorPtr);
					if (CounterpartActor != nullptr)
					{
						ActivePreviewActors.Add(CounterpartActor);
					}
				}
				else
				{
					ActivePreviewActors.Add(PreviewActor);
				}
			}
		}
	}
	else
	{
		ActivePreviewActors = PreviewActors;
	}

	SetActorLock(CurrentLockCameraIndex);
}

/*
 * SMediaFrameworkCaptureRenderTargetWidget implementation
 */
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMediaFrameworkCaptureRenderTargetWidget::Construct(const FArguments& InArgs)
{
	RenderTarget = InArgs._RenderTarget;

	SMediaFrameworkCaptureOutputWidget::FArguments BaseArguments;
	BaseArguments._Owner = InArgs._Owner;
	BaseArguments._MediaOutput = InArgs._MediaOutput;
	BaseArguments._CaptureOptions = InArgs._CaptureOptions;
	SMediaFrameworkCaptureOutputWidget::Construct(BaseArguments);

	TSharedRef<SWidget> PictureBox = SNew(SMediaImage, RenderTarget.Get());

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(MediaFrameworkUtilities::Padding)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				BuildBaseWidget(PictureBox, TEXT("Render Target Capture"))
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SMediaFrameworkCaptureRenderTargetWidget::~SMediaFrameworkCaptureRenderTargetWidget()
{
	StopOutput();
}

void SMediaFrameworkCaptureRenderTargetWidget::StartOutput()
{
	UMediaOutput* MediaOutputPtr = MediaOutput.Get();
	UTextureRenderTarget2D* RenderTargetPtr = RenderTarget.Get();
	if (MediaOutputPtr && RenderTargetPtr)
	{
		MediaCapture.Reset(MediaOutputPtr->CreateMediaCapture());
		if (MediaCapture.IsValid())
		{
			MediaCapture->CaptureTextureRenderTarget2D(RenderTargetPtr, CaptureOptions);
		}
	}
}

bool SMediaFrameworkCaptureRenderTargetWidget::IsValid() const
{
	return RenderTarget.IsValid();
}

/*
 * SMediaFrameworkCaptureCurrentViewportWidget implementation
 */
SMediaFrameworkCaptureCurrentViewportWidget::FPreviousViewportClientFlags::FPreviousViewportClientFlags()
	: bRealTime(false)
	, bSetListenerPosition(false)
	, bDrawAxes(false)
	, bDisableInput(false)
	, bAllowCinematicControl(false)
	, EngineShowFlags(ESFIM_Editor)
	, LastEngineShowFlags(ESFIM_Game)
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMediaFrameworkCaptureCurrentViewportWidget::Construct(const FArguments& InArgs)
{
	ViewMode = InArgs._ViewMode;

	SMediaFrameworkCaptureOutputWidget::FArguments BaseArguments;
	BaseArguments._Owner = InArgs._Owner;
	BaseArguments._MediaOutput = InArgs._MediaOutput;
	BaseArguments._CaptureOptions = InArgs._CaptureOptions;
	SMediaFrameworkCaptureOutputWidget::Construct(BaseArguments);

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(MediaFrameworkUtilities::Padding)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				BuildBaseWidget(SNullWidget::NullWidget, TEXT("Current Viewport Capture"))
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SMediaFrameworkCaptureCurrentViewportWidget::~SMediaFrameworkCaptureCurrentViewportWidget()
{
	StopOutput();
}

void SMediaFrameworkCaptureCurrentViewportWidget::StartOutput()
{
	ShutdownViewport();

	UMediaOutput* MediaOutputPtr = MediaOutput.Get();
	if (MediaOutputPtr)
	{
		MediaCapture.Reset(MediaOutputPtr->CreateMediaCapture());
		if (MediaCapture.IsValid())
		{
			TSharedPtr<FSceneViewport> SceneViewport;

			// Is it a "standalone" window
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				if (Context.WorldType == EWorldType::PIE)
				{
					UEditorEngine* EditorEngine = CastChecked<UEditorEngine>(GEngine);
					FSlatePlayInEditorInfo& Info = EditorEngine->SlatePlayInEditorMap.FindChecked(Context.ContextHandle);
					if (Info.SlatePlayInEditorWindowViewport.IsValid())
					{
						SceneViewport = Info.SlatePlayInEditorWindowViewport;
					}
				}
			}

			if (!SceneViewport.IsValid())
			{
				// Find a editor viewport
				FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
				TSharedPtr<IAssetViewport> ViewportInterface = LevelEditorModule.GetFirstActiveViewport();
				if (ViewportInterface.IsValid())
				{
					LevelViewport = ViewportInterface;
					SceneViewport = ViewportInterface->GetSharedActiveViewport();

					FEditorViewportClient& ViewportClient = ViewportInterface->GetAssetViewportClient();

					// Save settings
					ViewportClientFlags.bRealTime = ViewportClient.IsRealtime();
					ViewportClientFlags.bSetListenerPosition = ViewportClient.bSetListenerPosition;
					ViewportClientFlags.bDrawAxes = ViewportClient.bDrawAxes;
					ViewportClientFlags.bDisableInput = ViewportClient.bDisableInput;
					ViewportClientFlags.bAllowCinematicControl = ViewportClient.AllowsCinematicControl();
					ViewportClientFlags.VisibilityDelegate = ViewportClient.VisibilityDelegate;

					// Set settings for recording
					ViewportClient.SetRealtime(true);
					ViewportClient.bSetListenerPosition = false;
					ViewportClient.bDrawAxes = false;
					// Disable input by default unless we're dynamically rezizing the viewport texture as we capture.
					// Note that we originally disable input because click routing is broken when we resize the viewport through media capture at the moment.
					ViewportClient.bDisableInput = CaptureOptions.ResizeMethod != EMediaCaptureResizeMethod::ResizeInRenderPass ? true : ViewportClient.bDisableInput;
					ViewportClient.SetAllowCinematicControl(false);
					ViewportClient.VisibilityDelegate.BindLambda([] { return true; });
				}
			}

			if (SceneViewport.IsValid())
			{
				GEditor->OnLevelViewportClientListChanged().AddSP(this, &SMediaFrameworkCaptureCurrentViewportWidget::OnLevelViewportClientListChanged);
				EditorSceneViewport = SceneViewport;
				MediaCapture->OnStateChangedNative.AddSP(this, &SMediaFrameworkCaptureCurrentViewportWidget::OnMediaCaptureStateChanged);

				if (!MediaCapture->CaptureSceneViewport(SceneViewport, CaptureOptions))
				{
					ShutdownViewport();
				}
			}
		}
	}
}

void SMediaFrameworkCaptureCurrentViewportWidget::StopOutput()
{
	if (MediaCapture.IsValid())
	{
		MediaCapture->StopCapture(false);
	}
	ShutdownViewport();
}

void SMediaFrameworkCaptureCurrentViewportWidget::OnLevelViewportClientListChanged()
{
	bool bFound = false;
	TSharedPtr<FSceneViewport> EditorSceneViewportPtr = EditorSceneViewport.Pin();
	if (EditorSceneViewportPtr.IsValid())
	{
		for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
		{
			if (ViewportClient->Viewport == EditorSceneViewportPtr.Get())
			{
				bFound = true;
				break;
			}
		}
	}

	if (!bFound)
	{
		ShutdownViewport();
	}
}

void SMediaFrameworkCaptureCurrentViewportWidget::OnMediaCaptureStateChanged()
{
	if (MediaCapture.IsValid())
	{
		EMediaCaptureState State = MediaCapture->GetState();
		if (State != EMediaCaptureState::Capturing && State != EMediaCaptureState::Preparing)
		{
			ShutdownViewport();
		}
	}
}

void SMediaFrameworkCaptureCurrentViewportWidget::OnPrePIE()
{
	StopOutput();
}

void SMediaFrameworkCaptureCurrentViewportWidget::OnPrePIEEnded()
{
	StopOutput();
}

void SMediaFrameworkCaptureCurrentViewportWidget::ShutdownViewport()
{
	GEditor->OnLevelViewportClientListChanged().RemoveAll(this);

	TSharedPtr<FSceneViewport> EditorSceneViewportPin = EditorSceneViewport.Pin();
	TSharedPtr<IAssetViewport> LevelViewportPin = LevelViewport.Pin();
	if (LevelViewportPin.IsValid() && LevelViewportPin->GetSharedActiveViewport() == EditorSceneViewportPin)
	{
		FEditorViewportClient& ViewportClient = LevelViewportPin->GetAssetViewportClient();

		// Reset settings
		ViewportClient.SetRealtime(ViewportClientFlags.bRealTime);
		ViewportClient.bSetListenerPosition = ViewportClientFlags.bSetListenerPosition;
		ViewportClient.bDrawAxes = ViewportClientFlags.bDrawAxes;
		ViewportClient.bDisableInput = ViewportClientFlags.bDisableInput;
		ViewportClient.SetAllowCinematicControl(ViewportClientFlags.bAllowCinematicControl);
		ViewportClient.VisibilityDelegate = ViewportClientFlags.VisibilityDelegate;
	}

	LevelViewport.Reset();
	EditorSceneViewport.Reset();
	MediaCapture.Reset();
}

#undef LOCTEXT_NAMESPACE
