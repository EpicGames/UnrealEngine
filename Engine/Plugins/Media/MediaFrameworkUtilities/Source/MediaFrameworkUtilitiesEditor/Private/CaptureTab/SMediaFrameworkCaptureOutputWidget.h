// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "EditorViewportClient.h"
#include "MediaCapture.h"
#include "UObject/StrongObjectPtr.h"

class UMediaOutput;


class FLevelEditorViewportClient;
class FSceneViewport;
struct FSlateBrush;
class IAssetViewport;
class SViewport;
class SMediaFrameworkCapture;


namespace MediaFrameworkUtilities
{
	class FMediaFrameworkCaptureLevelEditorViewportClient;
}


/*
 * SMediaFrameworkCaptureOutputWidget definition
 */
class SMediaFrameworkCaptureOutputWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaFrameworkCaptureOutputWidget) {}
		SLATE_ARGUMENT(TWeakPtr<SMediaFrameworkCapture>, Owner)
		SLATE_ARGUMENT(TWeakObjectPtr<UMediaOutput>, MediaOutput)
		SLATE_ARGUMENT(FMediaCaptureOptions, CaptureOptions)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual void StopOutput();

	TSharedRef<SWidget> BuildBaseWidget(TSharedRef<SWidget> InnerWidget, const FString& CaptureType);

	FSlateColor HandleIconColorAndOpacity() const;
	const FSlateBrush* HandleIconImage() const;
	FText HandleIconText() const;
	virtual bool IsValid() const;

	virtual void OnPrePIE() {};
	virtual void OnPostPIEStarted() {};
	virtual void OnPrePIEEnded() {};

	TWeakPtr<SMediaFrameworkCapture> Owner;
	TWeakObjectPtr<UMediaOutput> MediaOutput;
	FMediaCaptureOptions CaptureOptions;

	TStrongObjectPtr<UMediaCapture> MediaCapture;
};

/*
 * SMediaFrameworkCaptureCameraViewportWidget definition
 */
class SMediaFrameworkCaptureCameraViewportWidget : public SMediaFrameworkCaptureOutputWidget
{
private:
	using Super = SMediaFrameworkCaptureOutputWidget;

public:
	SLATE_BEGIN_ARGS(SMediaFrameworkCaptureCameraViewportWidget) { }
		SLATE_ARGUMENT(TWeakPtr<SMediaFrameworkCapture>, Owner)
		SLATE_ARGUMENT(TWeakObjectPtr<UMediaOutput>, MediaOutput)
		SLATE_ARGUMENT(FMediaCaptureOptions, CaptureOptions)
		SLATE_ARGUMENT(TArray<TWeakObjectPtr<AActor>>, PreviewActors)
		SLATE_ARGUMENT(EViewModeIndex, ViewMode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SMediaFrameworkCaptureCameraViewportWidget();

	void StartOutput();
	virtual void StopOutput() override;

	virtual void OnPostPIEStarted() override;
	virtual void OnPrePIEEnded() override;

private:
	void SetActorLock(int32 CameraIndex);
	void UpdateActivePreviewList(bool bIsPIE);

	TSharedPtr<MediaFrameworkUtilities::FMediaFrameworkCaptureLevelEditorViewportClient> LevelViewportClient;
	TSharedPtr<SViewport> ViewportWidget;
	TSharedPtr<FSceneViewport> SceneViewport;

	TArray<TWeakObjectPtr<AActor>> PreviewActors;
	TArray<TWeakObjectPtr<AActor>> ActivePreviewActors;
	EViewModeIndex ViewMode;
	int32 CurrentLockCameraIndex;
};

/*
 * SMediaFrameworkCaptureRenderTargetWidget definition
 */
class SMediaFrameworkCaptureRenderTargetWidget : public SMediaFrameworkCaptureOutputWidget
{
private:
	using Super = SMediaFrameworkCaptureOutputWidget;

public:
	SLATE_BEGIN_ARGS(SMediaFrameworkCaptureRenderTargetWidget) {}
		SLATE_ARGUMENT(TWeakPtr<SMediaFrameworkCapture>, Owner)
		SLATE_ARGUMENT(TWeakObjectPtr<UMediaOutput>, MediaOutput)
		SLATE_ARGUMENT(FMediaCaptureOptions, CaptureOptions)
		SLATE_ARGUMENT(TWeakObjectPtr<UTextureRenderTarget2D>, RenderTarget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SMediaFrameworkCaptureRenderTargetWidget();

	void StartOutput();

	virtual bool IsValid() const override;

private:
	TWeakObjectPtr<UTextureRenderTarget2D> RenderTarget;
	TStrongObjectPtr<UMaterial> ImageMaterial;
	TSharedPtr<FSlateBrush> ImageMaterialBrush;
};

/*
 * SMediaFrameworkCaptureCurrentViewportWidget definition
 */
class SMediaFrameworkCaptureCurrentViewportWidget : public SMediaFrameworkCaptureOutputWidget
{
private:
	using Super = SMediaFrameworkCaptureOutputWidget;

public:
	SLATE_BEGIN_ARGS(SMediaFrameworkCaptureCurrentViewportWidget) {}
		SLATE_ARGUMENT(TWeakPtr<SMediaFrameworkCapture>, Owner)
		SLATE_ARGUMENT(TWeakObjectPtr<UMediaOutput>, MediaOutput)
		SLATE_ARGUMENT(FMediaCaptureOptions, CaptureOptions)
		SLATE_ARGUMENT(EViewModeIndex, ViewMode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SMediaFrameworkCaptureCurrentViewportWidget();
	virtual void StopOutput() override;

	void StartOutput();

	virtual void OnPrePIE() override;
	virtual void OnPrePIEEnded() override;

	struct FPreviousViewportClientFlags
	{
		FPreviousViewportClientFlags();
		bool bRealTime;
		bool bSetListenerPosition;
		bool bDrawAxes;
		bool bDisableInput;
		bool bAllowCinematicControl;
		FEngineShowFlags EngineShowFlags;
		FEngineShowFlags LastEngineShowFlags;
		FViewportStateGetter VisibilityDelegate;
	};

private:
	void ShutdownViewport();
	void OnLevelViewportClientListChanged();
	void OnMediaCaptureStateChanged();

private:
	TWeakPtr<FSceneViewport> EditorSceneViewport;
	TWeakPtr<IAssetViewport> LevelViewport;
	FPreviousViewportClientFlags ViewportClientFlags;

	EViewModeIndex ViewMode;
};
