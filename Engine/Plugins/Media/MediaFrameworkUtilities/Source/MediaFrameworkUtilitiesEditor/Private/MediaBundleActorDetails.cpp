// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaBundleActorDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SButton.h"

#include "MediaBundleActorBase.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "MediaBundleActorDetails"

TSharedRef<IDetailCustomization> FMediaBundleActorDetails::MakeInstance()
{
	return MakeShareable( new FMediaBundleActorDetails);
}

void FMediaBundleActorDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	IDetailCategoryBuilder& MediaBundleCategory = DetailBuilder.EditCategory( "MediaBundle" );

	const bool bForAdvanced = false;
	const FText PlayTextString = LOCTEXT("PlayMedia", "Request Play Media");
	const FText CloseTextString = LOCTEXT("CloseMedia", "Request Close Media");

	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	TSharedPtr<TArray<TWeakObjectPtr<AMediaBundleActorBase>>> ActorsListPtr = MakeShared<TArray<TWeakObjectPtr<AMediaBundleActorBase>>>();
	ActorsListPtr->Reserve(Objects.Num());
	for(TWeakObjectPtr<UObject>& Obj : Objects)
	{
		TWeakObjectPtr<AMediaBundleActorBase> ActorPtr = Cast<AMediaBundleActorBase>(Obj.Get());
		if (ActorPtr.IsValid())
		{
			ActorsListPtr->Add(ActorPtr);
		}
	}
	

	MediaBundleCategory.AddCustomRow(PlayTextString, bForAdvanced )
	.NameContent()
	[
		SNullWidget::NullWidget
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	.MaxDesiredWidth(250)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.Text(PlayTextString)
			.IsEnabled_Lambda([ActorsListPtr]()
			{
				return ActorsListPtr->ContainsByPredicate([&](const TWeakObjectPtr<AMediaBundleActorBase>& Other) 
				{ 
					return (Other.IsValid() ? !Other->IsPlayRequested() : false); 
				});
			})
			.OnClicked_Lambda([ActorsListPtr]() -> FReply
			{
				for(auto& ActorPtr : *ActorsListPtr)
				{
					if (AMediaBundleActorBase* Actor = ActorPtr.Get())
					{
						Actor->RequestOpenMediaSource();
					}
				}
				return FReply::Handled();
			})
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4, 0, 0, 0)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.Text(CloseTextString)
			.IsEnabled_Lambda([ActorsListPtr]()
			{
				return ActorsListPtr->ContainsByPredicate([&](const TWeakObjectPtr<AMediaBundleActorBase>& Other) 
				{
					return (Other.IsValid() ? Other->IsPlayRequested() : false); 
				});
			})
			.OnClicked(FOnClicked::CreateLambda([ActorsListPtr]()
			{
				for (auto& ActorPtr : *ActorsListPtr)
				{
					if (AMediaBundleActorBase* Actor = ActorPtr.Get())
					{
						Actor->RequestCloseMediaSource();
					}
				}
				return FReply::Handled();
			}))
		]
	];
}

#undef LOCTEXT_NAMESPACE
