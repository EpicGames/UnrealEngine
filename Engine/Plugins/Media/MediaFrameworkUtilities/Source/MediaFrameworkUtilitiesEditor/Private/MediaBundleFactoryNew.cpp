// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaBundleFactoryNew.h"

#include "AssetRegistry/AssetData.h"
#include "AssetTypeCategories.h"
#include "EngineAnalytics.h"
#include "IAnalyticsProviderET.h"
#include "MediaBundle.h"

#define LOCTEXT_NAMESPACE "MediaBundleFactoryNew"


/* UMediaBundleFactoryNew structors
 *****************************************************************************/

UMediaBundleFactoryNew::UMediaBundleFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMediaBundle::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UMediaBundleFactoryNew UFactory interface
 *****************************************************************************/

/**
 * @EventName MediaFramework.CreateNewMediaBundle
 * @Trigger Triggered when a media bundle asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* UMediaBundleFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UMediaBundle* NewMediaBundle = NewObject<UMediaBundle>(InParent, InClass, InName, Flags);
	NewMediaBundle->CreateInternalsEditor();
	
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.CreateNewMediaBundle"));
	}

	return NewMediaBundle;
}

uint32 UMediaBundleFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}

bool UMediaBundleFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

/* UActorFactoryMediaBundle structors
*****************************************************************************/

UActorFactoryMediaBundle::UActorFactoryMediaBundle(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("FactoryMediaBundleDisplayName", "Media Bundle Actor");
	NewActorClass = AMediaBundleActorBase::StaticClass();
	bUseSurfaceOrientation = false;
}


/* UActorFactoryMediaBundle UFactory interface
*****************************************************************************/
bool UActorFactoryMediaBundle::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.GetClass()->IsChildOf(UMediaBundle::StaticClass()))
	{
		OutErrorMsg = LOCTEXT("NoMediaBundle", "A valid Media Bundle must be specified.");
		return false;
	}

	return true;
}

void UActorFactoryMediaBundle::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UMediaBundle* MediaBundle = CastChecked<UMediaBundle>(Asset);
	AMediaBundleActorBase* MediaBundleActor = CastChecked<AMediaBundleActorBase>(NewActor);

	FProperty* MediaBundleProperty = FindFieldChecked<FProperty>(AMediaBundleActorBase::StaticClass(), "MediaBundle");
	FEditPropertyChain PropertyChain;
	PropertyChain.AddHead(MediaBundleProperty);
	static_cast<UObject*>(NewActor)->PreEditChange(PropertyChain);

	MediaBundleActor->MediaBundle = MediaBundle;

	FPropertyChangedEvent PropertyEvent(MediaBundleProperty);
	NewActor->PostEditChangeProperty(PropertyEvent);
}

UObject* UActorFactoryMediaBundle::GetAssetFromActorInstance(AActor* ActorInstance)
{
	check(ActorInstance->IsA(NewActorClass));
	AMediaBundleActorBase* MBA = CastChecked<AMediaBundleActorBase>(ActorInstance);

	return MBA->GetMediaBundle();
}

AActor* UActorFactoryMediaBundle::GetDefaultActor(const FAssetData& AssetData)
{
	if (UMediaBundle* Bundle = Cast<UMediaBundle>(AssetData.GetAsset()))
	{
		if (UClass* ActorClass = Bundle->MediaBundleActorClass.Get())
		{
			return ActorClass->GetDefaultObject<AActor>();
		}
	}
	return Super::GetDefaultActor(AssetData);
}

#undef LOCTEXT_NAMESPACE
