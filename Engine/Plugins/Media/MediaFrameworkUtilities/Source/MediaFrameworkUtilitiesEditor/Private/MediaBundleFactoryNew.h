// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ActorFactories/ActorFactory.h"
#include "Factories/Factory.h"


#include "MediaBundleFactoryNew.generated.h"

/**
 * Implements a factory for UMediaPlayer objects.
 */
UCLASS(hideCategories =Object)
class UMediaBundleFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
	//~ End UFactory Interface	
};

UCLASS(MinimalAPI, config=Editor, collapseCategories, hideCategories=Object)
class UActorFactoryMediaBundle : public UActorFactory
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	virtual AActor* GetDefaultActor(const FAssetData& AssetData) override;
	//~ End UActorFactory Interface	
};



