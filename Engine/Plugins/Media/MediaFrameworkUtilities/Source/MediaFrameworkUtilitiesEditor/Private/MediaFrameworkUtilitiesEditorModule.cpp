// Copyright Epic Games, Inc. All Rights Reserved.


#include "MediaFrameworkUtilitiesEditorModule.h"

#include "IMediaProfileEditorModule.h"
#include "LevelEditor.h"
#include "PropertyEditorModule.h"

#include "AssetEditor/MediaProfileCommands.h"
#include "CaptureTab/SMediaFrameworkCapture.h"
#include "MediaBundleActorDetails.h"
#include "MediaBundleActorBase.h"
#include "MediaBundleFactoryNew.h"
#include "MediaFrameworkUtilitiesPlacement.h"
#include "MediaProfileCaptureMenuExtension.h"
#include "Profile/MediaProfileCustomization.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfileBlueprintLibrary.h"
#include "Profile/MediaProfileSettings.h"
#include "Subsystems/PlacementSubsystem.h"
#include "VideoInputTab/SMediaFrameworkVideoInput.h"
#include "UI/MediaProfileMenuEntry.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"



#define LOCTEXT_NAMESPACE "MediaFrameworkEditor"

DEFINE_LOG_CATEGORY(LogMediaFrameworkUtilitiesEditor);

/**
 * Implements the MediaPlayerEditor module.
 */
class FMediaFrameworkUtilitiesEditorModule : public IModuleInterface
{
public:
	FName NotificationBarIdentifier = TEXT("MediaProfile");

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		if (GEditor)
		{
			FMediaProfileCommands::Register();
			FMediaFrameworkUtilitiesEditorStyle::Register();

			ActorFactoryMediaBundle = TStrongObjectPtr<UActorFactoryMediaBundle>(NewObject<UActorFactoryMediaBundle>());

			GEditor->ActorFactories.Add(ActorFactoryMediaBundle.Get());
			if (UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>())
			{
				PlacementSubsystem->RegisterAssetFactory(ActorFactoryMediaBundle.Get());
			}

			FMediaFrameworkUtilitiesPlacement::RegisterPlacement();

			// register detail panel customization
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.RegisterCustomClassLayout(AMediaBundleActorBase::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FMediaBundleActorDetails::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(UMediaProfile::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FMediaProfileCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(UMediaProfileSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FMediaProfileSettingsCustomization::MakeInstance));

			RegisterSection(PropertyModule,
				UMediaProfile::StaticClass(),
				TEXT("TimecodeProvider"),
				LOCTEXT("TimecodeProviderSection", "Timecode Provider"),
				TArray<FName> { TEXT("Timecode Provider") });

			RegisterSection(PropertyModule,
				UMediaProfile::StaticClass(),
				TEXT("Genlock"),
				LOCTEXT("GenlockSection", "Genlock"),
				TArray<FName> { TEXT("Genlock") });
			
			{
				const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
				TSharedRef<FWorkspaceItem> MediaBrowserGroup = MenuStructure.GetLevelEditorMediaCategory();

				SMediaFrameworkCapture::RegisterNomadTabSpawner(MediaBrowserGroup);
				SMediaFrameworkVideoInput::RegisterNomadTabSpawner(MediaBrowserGroup);
			}
			FMediaProfileMenuEntry::Register();

			{
				FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
				if (LevelEditorModule != nullptr)
				{
					FLevelEditorModule::FTitleBarItem Item;
					Item.Label = LOCTEXT("MediaProfileLabel", "MediaProfile: ");
					Item.Value = MakeAttributeLambda([]() { UObject* MediaProfile = UMediaProfileBlueprintLibrary::GetMediaProfile(); return MediaProfile ? FText::FromName(MediaProfile->GetFName()) : FText::GetEmpty(); });
					Item.Visibility = MakeAttributeLambda([]() { return GetDefault<UMediaProfileEditorSettings>()->bDisplayInMainEditor ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed; });
					LevelEditorModule->AddTitleBarItem(NotificationBarIdentifier, Item);
				}
			}

			UE::MediaFrameworkUtilities::Menus::ExtendMediaProfileDropdownMenu();
		}
	}

	virtual void ShutdownModule() override
	{
		if (!IsEngineExitRequested() && GEditor && UObjectInitialized())
		{
			FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
			if (LevelEditorModule != nullptr)
			{
				LevelEditorModule->RemoveTitleBarItem(NotificationBarIdentifier);
			}

			FMediaProfileMenuEntry::Unregister();
			SMediaFrameworkVideoInput::UnregisterNomadTabSpawner();
			SMediaFrameworkCapture::UnregisterNomadTabSpawner();

			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout(UMediaProfileSettings::StaticClass()->GetFName());
			PropertyModule.UnregisterCustomClassLayout(UMediaProfile::StaticClass()->GetFName());
			PropertyModule.UnregisterCustomClassLayout(AMediaBundleActorBase::StaticClass()->GetFName());

			FMediaFrameworkUtilitiesPlacement::UnregisterPlacement();

			GEditor->ActorFactories.RemoveAll([](const UActorFactory* ActorFactory) { return ActorFactory->IsA<UActorFactoryMediaBundle>(); });

			if (UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>())
			{
				PlacementSubsystem->UnregisterAssetFactory(ActorFactoryMediaBundle.Get());
			}

			FMediaFrameworkUtilitiesEditorStyle::Unregister();
			FMediaProfileCommands::Unregister();
		}
	}

private:
	void RegisterSection(FPropertyEditorModule& PropertyModule, UClass* InClass, FName InSectionName, FText InSectionLabel, const TArray<FName>& InCategoriesInSection)
	{
		const TSharedPtr<FPropertySection> Section = PropertyModule.FindOrCreateSection(InClass->GetFName(), InSectionName,InSectionLabel);
		for (const FName& Category : InCategoriesInSection)
		{
			Section->AddCategory(Category);
		}
	}
	
private:

	TStrongObjectPtr<UActorFactoryMediaBundle> ActorFactoryMediaBundle;
};


IMPLEMENT_MODULE(FMediaFrameworkUtilitiesEditorModule, MediaFrameworkUtilitiesEditor);


#undef LOCTEXT_NAMESPACE
