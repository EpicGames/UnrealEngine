// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaFrameworkUtilitiesPlacement.h"

#include "AssetThumbnail.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Views/SListView.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "IPlacementModeModule.h"

#include "MediaBundle.h"
#include "MediaSource.h"
#include "ThumbnailRendering/ThumbnailManager.h"


#define LOCTEXT_NAMESPACE "MediaFrameworkEditor"

/** The asset of the asset view */
struct FMediaPlacementListItem
{
	FMediaPlacementListItem() = default;

	bool IsValid() const
	{
		return MediaBundle.IsUAsset();
	}

	FText DisplayName;
	FAssetData MediaBundle;
};

/** The list view mode of the asset view */
class SMediaPlacementListView : public SListView<TSharedPtr<FMediaPlacementListItem>>
{
public:
	virtual bool SupportsKeyboardFocus() const override
	{
		return false;
	}
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override
	{
		return FReply::Unhandled();
	}
};

/** The Placement compound widget */
class SMediaPlacementPalette : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaPlacementPalette) {}
	SLATE_END_ARGS();

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct(const FArguments& InArgs)
	{
		BuildList();

		TSharedRef<SMediaPlacementListView> ListViewWidget =
			SNew(SMediaPlacementListView)
			.SelectionMode(ESelectionMode::Single)
			.ListItemsSource(&PlacementList)
			.OnGenerateRow(this, &SMediaPlacementPalette::MakeListViewWidget)
			.OnSelectionChanged(this, &SMediaPlacementPalette::OnSelectionChanged);

		ChildSlot
		[
			SNew(SScrollBorder, ListViewWidget)
			[
				ListViewWidget
			]
		];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	TSharedRef<ITableRow> MakeListViewWidget(TSharedPtr<FMediaPlacementListItem> MediaPlacement, const TSharedRef<STableViewBase>& OwnerTable)
	{
		if (!MediaPlacement.IsValid() || !MediaPlacement->IsValid())
		{
			return SNew(STableRow<TSharedPtr<FMediaPlacementListItem>>, OwnerTable);
		}

		TSharedRef< STableRow<TSharedPtr<FMediaPlacementListItem>> > TableRowWidget =
			SNew(STableRow<TSharedPtr<FMediaPlacementListItem>>, OwnerTable)
			.Style(FAppStyle::Get(), "PlacementBrowser.PlaceableItemRow")
			.OnDragDetected(this, &SMediaPlacementPalette::OnDraggingListViewWidget);

		// Get the MediaSource thumbnail or the MediaBundle is not loaded
		FAssetData ThumbnailAssetData = MediaPlacement->MediaBundle;
		if (MediaPlacement->MediaBundle.IsAssetLoaded())
		{
			UMediaBundle* MediaBundle = Cast<UMediaBundle>(MediaPlacement->MediaBundle.GetAsset());
			if (MediaBundle)
			{
				ThumbnailAssetData = FAssetData(MediaBundle->MediaSource);
			}
		}
		TSharedPtr<FAssetThumbnail> Thumbnail = MakeShareable(new FAssetThumbnail(ThumbnailAssetData, 32, 32, UThumbnailManager::Get().GetSharedThumbnailPool()));

		// Create the TableRow content
		TSharedRef<SWidget> Content =
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("NoBorder"))
			.Padding(0)
			.Cursor(EMouseCursor::GrabHand)
			[
				SNew(SHorizontalBox)
				// Icon
				+ SHorizontalBox::Slot()
				.Padding(0)
				.AutoWidth()
				[
					SNew(SBorder)
					.Padding(4.0f)
					.BorderImage(FAppStyle::GetBrush("ContentBrowser.ThumbnailShadow"))
					[
						SNew(SBox)
						.WidthOverride(35.0f)
						.HeightOverride(35.0f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ContentBrowser.ThumbnailShadow"))
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								Thumbnail->MakeThumbnailWidget()
							]
						]
					]
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(2, 0, 4, 0)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(0, 0, 0, 1)
					.AutoHeight()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "PlacementBrowser.Asset.Name")
						.Text(MediaPlacement->DisplayName)
					]
				]
			];

		TableRowWidget->SetContent(Content);

		return TableRowWidget;

	}

	void OnSelectionChanged(TSharedPtr<FMediaPlacementListItem> MediaPlacement, ESelectInfo::Type SelectionType)
	{
		SelectedMediaPlacement = MediaPlacement;
	}

	FReply OnDraggingListViewWidget(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			if (SelectedMediaPlacement.IsValid())
			{
				// We have an active brush builder, start a drag-drop
				TArray<FAssetData> InAssetData;
				InAssetData.Add(SelectedMediaPlacement->MediaBundle);
				return FReply::Handled().BeginDragDrop(FAssetDragDropOp::New(InAssetData));
			}
		}

		return FReply::Unhandled();
	}

	void BuildList()
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		TArray<FAssetData> AssetDatas;
		AssetRegistryModule.Get().GetAssetsByClass(FTopLevelAssetPath(TEXT("/Script/MediaFrameworkUtilities"), TEXT("MediaBundle")), AssetDatas, true);
		PlacementList.Reset(AssetDatas.Num());

		for (const FAssetData& AssetData : AssetDatas)
		{
			TSharedPtr<FMediaPlacementListItem> NewItem = MakeShared<FMediaPlacementListItem>();
			NewItem->DisplayName = FText::FromName(AssetData.AssetName);
			NewItem->MediaBundle = AssetData;
			PlacementList.Add(NewItem);
		}
	}

	TSharedPtr<FMediaPlacementListItem> SelectedMediaPlacement;
	TArray<TSharedPtr<FMediaPlacementListItem>> PlacementList;
};

void FMediaFrameworkUtilitiesPlacement::RegisterPlacement()
{
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	FName CategoryName = "Media";
	FPlacementCategoryInfo CategoryInfo(LOCTEXT("PlacementMode_Media", "Media"), FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("SequenceRecorder.TabIcon")), CategoryName, TEXT("PMMedia"), 35);
	CategoryInfo.CustomGenerator = []() -> TSharedRef<SWidget> { return SNew(SMediaPlacementPalette); };
	PlacementModeModule.RegisterPlacementCategory(CategoryInfo);
}

void FMediaFrameworkUtilitiesPlacement::UnregisterPlacement()
{
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	PlacementModeModule.UnregisterPlacementCategory(TEXT("PMMedia"));
}

#undef LOCTEXT_NAMESPACE
