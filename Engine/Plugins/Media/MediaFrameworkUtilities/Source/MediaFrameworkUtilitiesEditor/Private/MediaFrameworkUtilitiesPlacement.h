// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FMediaFrameworkUtilitiesPlacement
{
public:
	static void RegisterPlacement();
	static void UnregisterPlacement();
};
