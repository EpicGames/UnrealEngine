// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MediaFrameworkVideoInputSettings.generated.h"

class UMediaBundle;
class UMediaSource;
class UMediaTexture;

USTRUCT()
struct FMediaFrameworkVideoInputSourceSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Media")
	TSoftObjectPtr<UMediaSource> MediaSource;

	UPROPERTY(EditAnywhere, Category="Media")
	TSoftObjectPtr<UMediaTexture> MediaTexture;
};

/**
 * Settings for the video input tab.
 */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UMediaFrameworkVideoInputSettings : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(config, EditAnywhere, Category="Media Bundle")
	TArray<TSoftObjectPtr<UMediaBundle>> MediaBundles;

	UPROPERTY(config, EditAnywhere, Category="Media Source")
	TArray<FMediaFrameworkVideoInputSourceSettings> MediaSources;

	UPROPERTY(config)
	bool bReopenMediaBundles;

	UPROPERTY(config)
	bool bReopenMediaSources;

	UPROPERTY(config)
	float ReopenDelay = 3.f;

	UPROPERTY(config)
	bool bIsVerticalSplitterOrientation = true;
};
