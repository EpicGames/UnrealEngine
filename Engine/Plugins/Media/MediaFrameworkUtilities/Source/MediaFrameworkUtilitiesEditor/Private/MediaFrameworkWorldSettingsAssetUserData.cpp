// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaFrameworkWorldSettingsAssetUserData.h"
#include "GameFramework/Actor.h"
#include "Serialization/CustomVersion.h"
#include "UObject/EnterpriseObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

FMediaFrameworkCaptureCurrentViewportOutputInfo::FMediaFrameworkCaptureCurrentViewportOutputInfo()
	: MediaOutput(nullptr)
	, ViewMode(VMI_Unknown)
{
}


FMediaFrameworkCaptureCameraViewportCameraOutputInfo::FMediaFrameworkCaptureCameraViewportCameraOutputInfo()
	: MediaOutput(nullptr)
	, ViewMode(VMI_Unknown)
{
}

bool FMediaFrameworkCaptureCameraViewportCameraOutputInfo::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	return false;
}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FMediaFrameworkCaptureCameraViewportCameraOutputInfo::PostSerialize(const FArchive& Ar)
{
	const int32 CustomVersion = Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID);
	if (Ar.IsLoading() && CustomVersion < FUE5ReleaseStreamObjectVersion::MediaProfilePluginCaptureCameraSoftPtr)
	{

		for (const TLazyObjectPtr<AActor>& LazyActor : LockedActors_DEPRECATED)
		{
			Cameras.Add(LazyActor.Get());
		}

		LockedActors_DEPRECATED.Empty();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

FMediaFrameworkCaptureRenderTargetCameraOutputInfo::FMediaFrameworkCaptureRenderTargetCameraOutputInfo()
	: RenderTarget(nullptr)
	, MediaOutput(nullptr)
{
}

FMediaFrameworkCaptureMediaTextureOutputInfo::FMediaFrameworkCaptureMediaTextureOutputInfo()
	: MediaTexture(nullptr)
	, MediaOutput(nullptr)
{
	CaptureOptions.bApplyLinearToSRGBConversion = true;
	CaptureOptions.ResizeMethod = EMediaCaptureResizeMethod::ResizeInRenderPass;
}

UMediaFrameworkWorldSettingsAssetUserData::UMediaFrameworkWorldSettingsAssetUserData()
{
	CurrentViewportMediaOutput.CaptureOptions.ResizeMethod = EMediaCaptureResizeMethod::ResizeSource;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UMediaFrameworkWorldSettingsAssetUserData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEnterpriseObjectVersion::GUID);
	if (Ar.IsLoading() && FEnterpriseObjectVersion::MediaFrameworkUserDataLazyObject > Ar.CustomVer(FEnterpriseObjectVersion::GUID))
	{
		for (FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo : ViewportCaptures)
		{
			if (OutputInfo.LockedCameraActors_DEPRECATED.Num() > 0)
			{
				for (AActor* Actor : OutputInfo.LockedCameraActors_DEPRECATED)
				{
					if (Actor)
					{
						OutputInfo.Cameras.Add(Actor);
					}
				}
				OutputInfo.LockedCameraActors_DEPRECATED.Empty();
			}
		}
	}
}

bool UE::MediaFrameworkWorldSettings::Helpers::HasAnyOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput)
{
	return
		FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(InSettings, InMediaOutput) ||
		FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(InSettings, InMediaOutput) ||
		FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(InSettings, InMediaOutput) ||
		FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(InSettings, InMediaOutput);
}

namespace UE::MediaFrameworkWorldSettings::Helpers
{
	template <>
	FMediaFrameworkCaptureCurrentViewportOutputInfo* FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings,
		UMediaOutput* InMediaOutput)
	{
		if (InSettings != nullptr)
		{
			if (InSettings->CurrentViewportMediaOutput.MediaOutput == InMediaOutput)
			{
				return &InSettings->CurrentViewportMediaOutput;
			}
		}
		
		return nullptr;
	}

	template <>
	FMediaFrameworkCaptureCameraViewportCameraOutputInfo* FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings,
		UMediaOutput* InMediaOutput)
	{
		if (InSettings != nullptr)
		{
			for (FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo : InSettings->ViewportCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					return &OutputInfo;
				}
			}
		}
		
		return nullptr;
	}

	template <>
	FMediaFrameworkCaptureRenderTargetCameraOutputInfo* FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings,
		UMediaOutput* InMediaOutput)
	{
		if (InSettings != nullptr)
		{
			for (FMediaFrameworkCaptureRenderTargetCameraOutputInfo& OutputInfo : InSettings->RenderTargetCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					return &OutputInfo;
				}
			}
		}
		
		return nullptr;
	}
	
	template <>
	FMediaFrameworkCaptureMediaTextureOutputInfo* FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings,
		UMediaOutput* InMediaOutput)
	{
		if (InSettings != nullptr)
		{
			for (FMediaFrameworkCaptureMediaTextureOutputInfo& OutputInfo : InSettings->MediaTextureCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					return &OutputInfo;
				}
			}
		}
		
		return nullptr;
	}

	template <>
	TArray<FMediaFrameworkCaptureCurrentViewportOutputInfo>	FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings,
		UMediaOutput* InMediaOutput)
	{
		TArray<FMediaFrameworkCaptureCurrentViewportOutputInfo> OutputInfos;
		if (InSettings != nullptr)
		{
			if (InSettings->CurrentViewportMediaOutput.MediaOutput == InMediaOutput)
			{
				OutputInfos.Add(InSettings->CurrentViewportMediaOutput);
			}
		}
	
		return OutputInfos;
	}

	template <>
	TArray<FMediaFrameworkCaptureCameraViewportCameraOutputInfo> FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings,
		UMediaOutput* InMediaOutput)
	{
		TArray<FMediaFrameworkCaptureCameraViewportCameraOutputInfo> OutputInfos;
		if (InSettings != nullptr)
		{
			for (const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo : InSettings->ViewportCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					OutputInfos.Add(OutputInfo);
				}
			}
		}
	
		return OutputInfos;
	}

	template <>
	TArray<FMediaFrameworkCaptureRenderTargetCameraOutputInfo> FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings,
		UMediaOutput* InMediaOutput)
	{
		TArray<FMediaFrameworkCaptureRenderTargetCameraOutputInfo> OutputInfos;
		if (InSettings != nullptr)
		{
			for (const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& OutputInfo : InSettings->RenderTargetCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					OutputInfos.Add(OutputInfo);
				}
			}
		}
	
		return OutputInfos;
	}

	template <>
	TArray<FMediaFrameworkCaptureMediaTextureOutputInfo> FindAllOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings,
		UMediaOutput* InMediaOutput)
	{
		TArray<FMediaFrameworkCaptureMediaTextureOutputInfo> OutputInfos;
		if (InSettings != nullptr)
		{
			for (const FMediaFrameworkCaptureMediaTextureOutputInfo& OutputInfo : InSettings->MediaTextureCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					OutputInfos.Add(OutputInfo);
				}
			}
		}
	
		return OutputInfos;
	}

	template <>
	void ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput,
		TFunction<void(FMediaFrameworkCaptureCurrentViewportOutputInfo&)> ForEachFunc)
	{
		TArray<FMediaFrameworkCaptureCurrentViewportOutputInfo> OutputInfos;
		if (InSettings != nullptr)
		{
			if (InSettings->CurrentViewportMediaOutput.MediaOutput == InMediaOutput)
			{
				ForEachFunc(InSettings->CurrentViewportMediaOutput);
			}
		}
	}

	template <>
	void ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput,
		TFunction<void(FMediaFrameworkCaptureCameraViewportCameraOutputInfo&)> ForEachFunc)
	{
		TArray<FMediaFrameworkCaptureCameraViewportCameraOutputInfo> OutputInfos;
		if (InSettings != nullptr)
		{
			for (FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo : InSettings->ViewportCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					ForEachFunc(OutputInfo);
				}
			}
		}
	}

	template <>
	void ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput,
		TFunction<void(FMediaFrameworkCaptureRenderTargetCameraOutputInfo&)> ForEachFunc)
	{
		TArray<FMediaFrameworkCaptureRenderTargetCameraOutputInfo> OutputInfos;
		if (InSettings != nullptr)
		{
			for (FMediaFrameworkCaptureRenderTargetCameraOutputInfo& OutputInfo : InSettings->RenderTargetCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					ForEachFunc(OutputInfo);
				}
			}
		}
	}

	template <>
	void ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput,
		TFunction<void(FMediaFrameworkCaptureMediaTextureOutputInfo&)> ForEachFunc)
	{
		TArray<FMediaFrameworkCaptureMediaTextureOutputInfo> OutputInfos;
		if (InSettings != nullptr)
		{
			for (FMediaFrameworkCaptureMediaTextureOutputInfo& OutputInfo : InSettings->MediaTextureCaptures)
			{
				if (OutputInfo.MediaOutput == InMediaOutput)
				{
					ForEachFunc(OutputInfo);
				}
			}
		}
	}

	template <>
	int32 RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput)
	{
		if (InSettings->CurrentViewportMediaOutput.MediaOutput == InMediaOutput)
		{
			InSettings->CurrentViewportMediaOutput.MediaOutput = nullptr;
			return 1;
		}
		
		return 0;
	}

	template <>
	int32 RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput)
	{
		return InSettings->ViewportCaptures.RemoveAll([InMediaOutput](const FMediaFrameworkCaptureCameraViewportCameraOutputInfo& OutputInfo)
		{
			return OutputInfo.MediaOutput == InMediaOutput;
		});
	}

	template <>
	int32 RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput)
	{
		return InSettings->RenderTargetCaptures.RemoveAll([InMediaOutput](const FMediaFrameworkCaptureRenderTargetCameraOutputInfo& OutputInfo)
		{
			return OutputInfo.MediaOutput == InMediaOutput;
		});
	}

	template <>
	int32 RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput)
	{
		return InSettings->MediaTextureCaptures.RemoveAll([InMediaOutput](const FMediaFrameworkCaptureMediaTextureOutputInfo& OutputInfo)
		{
			return OutputInfo.MediaOutput == InMediaOutput;
		});
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
