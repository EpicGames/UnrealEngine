// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "MediaCapture.h"

#include "MediaFrameworkWorldSettingsAssetUserData.generated.h"

enum EViewModeIndex : int;

class UMediaFrameworkWorldSettingsAssetUserData;
class UMediaOutput;
class UTextureRenderTarget2D;

/**
 * FMediaFrameworkCaptureCurrentViewportOutputInfo
 */
USTRUCT()
struct FMediaFrameworkCaptureCurrentViewportOutputInfo
{
	GENERATED_BODY()

	FMediaFrameworkCaptureCurrentViewportOutputInfo();

	UPROPERTY(EditAnywhere, Category="MediaViewportCapture")
	TObjectPtr<UMediaOutput> MediaOutput;

	UPROPERTY(EditAnywhere, Category="MediaViewportCapture")
	FMediaCaptureOptions CaptureOptions;

	UPROPERTY()
	TEnumAsByte<EViewModeIndex> ViewMode;
};

/**
 * FMediaFrameworkCaptureCameraViewportCameraOutputInfo
 */
USTRUCT()
struct FMediaFrameworkCaptureCameraViewportCameraOutputInfo
{
	GENERATED_BODY()

	FMediaFrameworkCaptureCameraViewportCameraOutputInfo();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	// this is a requirement for clang to compile without warnings.
	~FMediaFrameworkCaptureCameraViewportCameraOutputInfo() = default;
	FMediaFrameworkCaptureCameraViewportCameraOutputInfo(const FMediaFrameworkCaptureCameraViewportCameraOutputInfo&) = default;
	FMediaFrameworkCaptureCameraViewportCameraOutputInfo(FMediaFrameworkCaptureCameraViewportCameraOutputInfo&&) = default;
	FMediaFrameworkCaptureCameraViewportCameraOutputInfo& operator=(const FMediaFrameworkCaptureCameraViewportCameraOutputInfo&) = default;
	FMediaFrameworkCaptureCameraViewportCameraOutputInfo& operator=(FMediaFrameworkCaptureCameraViewportCameraOutputInfo&&) = default;
	
	UE_DEPRECATED(5.7, "Use Cameras instead")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This property is no longer supported. Use Cameras instead"))
	TArray<TLazyObjectPtr<AActor>> LockedActors_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	UPROPERTY(EditAnywhere, Category="MediaViewportCapture")
	TArray<TSoftObjectPtr<AActor>> Cameras;

	UPROPERTY(EditAnywhere, Category="MediaViewportCapture")
	TObjectPtr<UMediaOutput> MediaOutput;

	UPROPERTY(EditAnywhere, Category="MediaViewportCapture")
	FMediaCaptureOptions CaptureOptions;

	UPROPERTY()
	TEnumAsByte<EViewModeIndex> ViewMode;

	bool Serialize(FArchive& Ar);
	
#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif
	
private:
	//DEPRECATED 4.21 The type of LockedCameraActors has changed and will be removed from the code base in a future release. Use LockedActors.
	UPROPERTY()
	TArray<TObjectPtr<AActor>> LockedCameraActors_DEPRECATED;
	friend UMediaFrameworkWorldSettingsAssetUserData;
};

template<> struct TStructOpsTypeTraits<FMediaFrameworkCaptureCameraViewportCameraOutputInfo> : public TStructOpsTypeTraitsBase2<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>
{
	enum
	{
		WithSerializer = true,

#if WITH_EDITORONLY_DATA
		WithPostSerialize = true,
#endif
	};
};

/**
 * FMediaFrameworkCaptureRenderTargetCameraOutputInfo
 */
USTRUCT()
struct FMediaFrameworkCaptureRenderTargetCameraOutputInfo
{
	GENERATED_BODY()

	FMediaFrameworkCaptureRenderTargetCameraOutputInfo();

	UPROPERTY(EditAnywhere, Category="MediaRenderTargetCapture")
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

	UPROPERTY(EditAnywhere, Category="MediaRenderTargetCapture")
	TObjectPtr<UMediaOutput> MediaOutput;

	UPROPERTY(EditAnywhere, Category="MediaRenderTargetCapture")
	FMediaCaptureOptions CaptureOptions;
};

USTRUCT()
struct FMediaFrameworkCaptureMediaTextureOutputInfo
{
	GENERATED_BODY()

	FMediaFrameworkCaptureMediaTextureOutputInfo();

	UPROPERTY(EditAnywhere, Category="Media Texture Capture")
	TObjectPtr<UMediaTexture> MediaTexture;

	UPROPERTY(EditAnywhere, Category="Media Texture Capture")
	FMediaCaptureTransform Transform;
	
	UPROPERTY(EditAnywhere, Category="Media Texture Capture")
	TObjectPtr<UMediaOutput> MediaOutput;

	UPROPERTY(EditAnywhere, Category="Media Texture Capture")
	FMediaCaptureOptions CaptureOptions;
};

/**
 * UMediaFrameworkCaptureCameraViewportAssetUserData
 */
UCLASS(MinimalAPI, config = Editor)
class UMediaFrameworkWorldSettingsAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UMediaFrameworkWorldSettingsAssetUserData();

	UPROPERTY(EditAnywhere, config, Category="Media Render Target Capture", meta=(ShowOnlyInnerProperties))
	TArray<FMediaFrameworkCaptureRenderTargetCameraOutputInfo> RenderTargetCaptures;

	UPROPERTY(EditAnywhere, config, Category="Media Viewport Capture", meta=(ShowOnlyInnerProperties))
	TArray<FMediaFrameworkCaptureCameraViewportCameraOutputInfo> ViewportCaptures;

	UPROPERTY(config)
	TArray<FMediaFrameworkCaptureMediaTextureOutputInfo> MediaTextureCaptures;
	
	/**
	 * Capture the current viewport. It may be the level editor active viewport or a PIE instance launch with "New Editor Window PIE".
	 * @note The behavior is different from MediaCapture.CaptureActiveSceneViewport. Here we can capture the editor viewport (since we are in the editor).
	 * @note If the viewport is the level editor active viewport, then all inputs will be disabled and the viewport will always rendered.
	 */
	UPROPERTY(EditAnywhere, config, Category="Media Current Viewport Capture", meta=(DisplayName="Current Viewport"))
	FMediaFrameworkCaptureCurrentViewportOutputInfo CurrentViewportMediaOutput;

public:
	virtual void Serialize(FArchive& Ar) override;
};

namespace UE::MediaFrameworkWorldSettings::Helpers
{
	/**
	 * Gets whether there is output info of any type for the specified media output in the capture settings
	 * @param InSettings The capture settings to search in
	 * @param InMediaOutput The media output to look for references for
	 * @return true if there is valid output info for the media output; otherwise, false
	 */
	bool HasAnyOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	
	/**
	 * Finds the first capture output info of a specific type in the capture settings that reference the specified media output
	 * @tparam TOutputInfo The type of capture output info to search for
	 * @param InSettings The capture settings to search in
	 * @param InMediaOutput The media output to look for references for
	 * @return A pointer to the found output info, or null if none were found
	 */
	template<typename TOutputInfo>
	TOutputInfo* FindFirstOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput) { return nullptr; }
	
	// Template specializations
	
	template<> FMediaFrameworkCaptureCurrentViewportOutputInfo* FindFirstOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> FMediaFrameworkCaptureCameraViewportCameraOutputInfo* FindFirstOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> FMediaFrameworkCaptureRenderTargetCameraOutputInfo* FindFirstOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> FMediaFrameworkCaptureMediaTextureOutputInfo* FindFirstOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	
	/**
	 * Finds all the capture output infos of a specific type in the capture settings that reference the specified media output
	 * @tparam TOutputInfo The type of capture output info to search for
	 * @param InSettings The capture settings to search in
	 * @param InMediaOutput The media output to look for references for
	 * @return A list of capture output infos of type TOutputInfo that reference InMediaOutput
	 */
	template<typename TOutputInfo>
	TArray<TOutputInfo> FindAllOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput) { return TArray<TOutputInfo>(); }
	
	// Template specializations
	
	template<> TArray<FMediaFrameworkCaptureCurrentViewportOutputInfo> FindAllOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> TArray<FMediaFrameworkCaptureCameraViewportCameraOutputInfo> FindAllOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> TArray<FMediaFrameworkCaptureRenderTargetCameraOutputInfo> FindAllOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> TArray<FMediaFrameworkCaptureMediaTextureOutputInfo> FindAllOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);

	/**
	 * Calls the specified function on each capture output info of the specified type in the capture settings that references the specified media output
	 * @tparam TOutputInfo The type of capture output info to search for
	 * @param InSettings The capture settings to search in
	 * @param InMediaOutput The media output to look for references for
	 * @param ForEachFunc The function to run on each matching capture output info
	 */
	template<typename TOutputInfo>
	void ForEachOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput, TFunction<void(TOutputInfo&)> ForEachFunc) { }
	
	// Template specializations
	
	template<> void ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput, TFunction<void(FMediaFrameworkCaptureCurrentViewportOutputInfo&)> ForEachFunc);
	template<> void ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput, TFunction<void(FMediaFrameworkCaptureCameraViewportCameraOutputInfo&)> ForEachFunc);
	template<> void ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput, TFunction<void(FMediaFrameworkCaptureRenderTargetCameraOutputInfo&)> ForEachFunc);
	template<> void ForEachOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput, TFunction<void(FMediaFrameworkCaptureMediaTextureOutputInfo&)> ForEachFunc);
	
	
	template<typename TOutputInfo>
	int32 RemoveAllOutputInfoForMediaOutput(UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput) { return 0; }
	
	// Template specializations
	
	template<> int32 RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> int32 RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> int32 RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
	template<> int32 RemoveAllOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(
		UMediaFrameworkWorldSettingsAssetUserData* InSettings, UMediaOutput* InMediaOutput);
}