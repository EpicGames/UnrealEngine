// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaProfileCaptureMenuExtension.h"

#include "IMediaProfileEditorModule.h"
#include "AssetEditor/MediaProfileEditorUserSettings.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "MediaProfileCaptureMenuExtension"

namespace UE::MediaFrameworkUtilities::Private
{
	bool IsMediaCapturing(int32 InMediaOutputIndex)
	{
		UMediaProfile* CurrentMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
		if (!CurrentMediaProfile)
		{
			return false;
		}
		
		return CurrentMediaProfile->GetPlaybackManager()->IsOutputCapturingFromIndex(InMediaOutputIndex);
	}
	
	void ToggleMediaCapture(int32 InMediaOutputIndex)
	{
		UMediaProfileEditorCaptureSettings* CaptureSettings = GetMutableDefault<UMediaProfileEditorCaptureSettings>();
		UMediaProfile* CurrentMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
		if (!CaptureSettings || !CurrentMediaProfile)
		{
			return;
		}

		UMediaOutput* MediaOutput = CurrentMediaProfile->GetMediaOutput(InMediaOutputIndex);
		if (!MediaOutput)
		{
			return;
		}

		UMediaProfilePlaybackManager* PlaybackManager = CurrentMediaProfile->GetPlaybackManager();
		if (!PlaybackManager)
		{
			return;
		}

		using namespace UE::MediaFrameworkWorldSettings::Helpers;

		if (!PlaybackManager->IsOutputCapturing(MediaOutput))
		{
			if (FMediaFrameworkCaptureCurrentViewportOutputInfo* CurrentViewportOutputInfo =
				FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCurrentViewportOutputInfo>(CaptureSettings, MediaOutput))
			{
				PlaybackManager->OpenActiveViewportOutput(MediaOutput, CurrentViewportOutputInfo->CaptureOptions, CaptureSettings->bAutoRestartCaptureOnChange);
			}

			if (FMediaFrameworkCaptureCameraViewportCameraOutputInfo* ViewportCapture =
				FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureCameraViewportCameraOutputInfo>(CaptureSettings, MediaOutput))
			{
				// Make sure there is a managed viewport before starting the capture
				TSharedPtr<FViewportClient> ViewportClient = PlaybackManager->GetOrCreateManagedViewport(MediaOutput, ViewportCapture->ViewMode);
				if (ViewportClient.IsValid())
				{
					PlaybackManager->OpenManagedViewportOutput(MediaOutput, ViewportCapture->CaptureOptions, CaptureSettings->bAutoRestartCaptureOnChange);
				}
			}

			if (FMediaFrameworkCaptureRenderTargetCameraOutputInfo* RenderTargetCapture =
				FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureRenderTargetCameraOutputInfo>(CaptureSettings, MediaOutput))
			{
				PlaybackManager->OpenRenderTargetOutput(MediaOutput, RenderTargetCapture->RenderTarget, RenderTargetCapture->CaptureOptions, CaptureSettings->bAutoRestartCaptureOnChange);
			}

			if (FMediaFrameworkCaptureMediaTextureOutputInfo* MediaTextureCapture =
				FindFirstOutputInfoForMediaOutput<FMediaFrameworkCaptureMediaTextureOutputInfo>(CaptureSettings, MediaOutput))
			{
				// For media texture capture, if the media texture is from a media profile source, make sure the source is open before starting capture
				int32 MediaSourceIndex = INDEX_NONE;
				if (PlaybackManager->IsValidSourceMediaTexture(MediaTextureCapture->MediaTexture, MediaSourceIndex))
				{
					PlaybackManager->OpenSourceFromIndex(MediaSourceIndex, CurrentMediaProfile);
				}
				
				PlaybackManager->OpenMediaTextureOutput(MediaOutput, MediaTextureCapture->MediaTexture, MediaTextureCapture->CaptureOptions, MediaTextureCapture->Transform, CaptureSettings->bAutoRestartCaptureOnChange);
			}
		}
		else
		{
			PlaybackManager->CloseOutputFromIndex(InMediaOutputIndex);
		}
	}
	
	
	void AddMediaCaptureMenuEntries(FMenuBuilder& InMenuBuilder)
	{
		UMediaProfileEditorCaptureSettings* CaptureSettings = GetMutableDefault<UMediaProfileEditorCaptureSettings>();
		UMediaProfile* CurrentMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
		if (!CaptureSettings || !CurrentMediaProfile)
		{
			return;
		}

		InMenuBuilder.BeginSection("Capture", LOCTEXT("CaptureSection", "Capture"));
		{
			const int32 NumOutputs = CurrentMediaProfile->NumMediaOutputs();
			for (int32 Index = 0; Index < NumOutputs; ++Index)
			{
				if (UMediaOutput* MediaOutput = CurrentMediaProfile->GetMediaOutput(Index))
				{
					if (UE::MediaFrameworkWorldSettings::Helpers::HasAnyOutputInfoForMediaOutput(CaptureSettings, MediaOutput))
					{
						InMenuBuilder.AddMenuEntry(
							FText::FromString(CurrentMediaProfile->GetLabelForMediaOutput(Index)),
							FText::GetEmpty(),
							FSlateIconFinder::FindIconForClass(MediaOutput->GetClass()),
							FUIAction(
								FExecuteAction::CreateStatic(&ToggleMediaCapture, Index),
								FCanExecuteAction(),
								FIsActionChecked::CreateStatic(&IsMediaCapturing, Index)),
							NAME_None,
							EUserInterfaceActionType::ToggleButton);
					}
				}
			}
		}
		InMenuBuilder.EndSection();
	}
}

void UE::MediaFrameworkUtilities::Menus::ExtendMediaProfileDropdownMenu()
{
	IMediaProfileEditorModule::Get().GetMediaProfileMenuExtender()->AddMenuExtension("MediaProfile", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateStatic(&Private::AddMediaCaptureMenuEntries));
}

#undef LOCTEXT_NAMESPACE
