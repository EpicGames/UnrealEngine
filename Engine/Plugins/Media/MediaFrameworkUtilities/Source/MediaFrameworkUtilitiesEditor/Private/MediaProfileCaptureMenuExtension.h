// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::MediaFrameworkUtilities::Menus
{
	/** Extends the media profile toolbar/dropdown menu to add media capture options */
	void ExtendMediaProfileDropdownMenu();
}