// Copyright Epic Games, Inc. All Rights Reserved.

#include "Profile/MediaProfileCustomization.h"

#include "DetailLayoutBuilder.h"
#include "MediaAssets/ProxyMediaOutput.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "PropertyHandle.h"


#define LOCTEXT_NAMESPACE "MediaProfileCustomization"

/**
 *
 */
TSharedRef<IDetailCustomization> FMediaProfileCustomization::MakeInstance()
{
	return MakeShareable(new FMediaProfileCustomization);
}


void FMediaProfileCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	Super::CustomizeDetails(DetailLayout);

	const TArray<UProxyMediaSource*> SourceProxies = IMediaProfileManager::Get().GetAllMediaSourceProxy();
	const TArray<UProxyMediaOutput*> OutputProxies = IMediaProfileManager::Get().GetAllMediaOutputProxy();

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailLayout.GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& CurrentWeakObject : SelectedObjects)
	{
		if (UMediaProfile* MediaProfile = Cast<UMediaProfile>(CurrentWeakObject.Get()))
		{
			MediaProfile->FixNumSourcesAndOutputs();
		}
	}

	for(int32 Index = 0; Index < SourceProxies.Num(); ++Index)
	{
		if (UProxyMediaSource* Proxy = SourceProxies[Index])
		{
			FName PropertyName = *FString::Printf(TEXT("MediaSources[%d]"), Index);
			TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(PropertyName);
			PropertyHandle->SetPropertyDisplayName(FText::FromName(Proxy->GetFName()));
		}
	}

	for (int32 Index = 0; Index < OutputProxies.Num(); ++Index)
	{
		if (UProxyMediaOutput* Proxy = OutputProxies[Index])
		{
			FName PropertyName = *FString::Printf(TEXT("MediaOutputs[%d]"), Index);
			TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(PropertyName);
			PropertyHandle->SetPropertyDisplayName(FText::FromName(Proxy->GetFName()));
		}
	}
}

#undef LOCTEXT_NAMESPACE
