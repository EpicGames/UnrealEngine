// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Profile/MediaProfileSettingsCustomization.h"

class IDetailCustomization;


/**
 *
 */
class FMediaProfileCustomization : public FMediaProfileSettingsCustomization
{
private:
	using Super = FMediaProfileSettingsCustomization;

public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	//~ End IDetailCustomization Interface
};
