// Copyright Epic Games, Inc. All Rights Reserved.

#include "Profile/MediaProfileSettingsCustomization.h"

#include "IAssetTools.h"
#include "Profile/IMediaProfileManager.h"
#include "Misc/PackageName.h"
#include "Profile/MediaProfileSettings.h"
#include "Profile/SMediaProfileSettingsOptionsWindow.h"

#include "MediaAssets/ProxyMediaOutput.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "MediaBundle.h"

#include "AssetToolsModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "FileHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/ScopedSlowTask.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SWindow.h"


#define LOCTEXT_NAMESPACE "MediaProfileSettings"

/**
 *
 */
TSharedRef<IDetailCustomization> FMediaProfileSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FMediaProfileSettingsCustomization);
}


void FMediaProfileSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const UMediaProfileSettings* MediaProfileSettings = GetDefault<UMediaProfileSettings>();
	check(MediaProfileSettings);

	const TArray<UProxyMediaOutput*> OutputProxies = MediaProfileSettings->LoadMediaOutputProxies();
	const TArray<UProxyMediaSource*> SourceProxies = MediaProfileSettings->LoadMediaSourceProxies();

	bNotConfigured = OutputProxies.Num() == 0 && SourceProxies.Num() == 0;
	bNeedFixup = false;
	for (const UProxyMediaSource* Proxy : SourceProxies)
	{
		if (Proxy == nullptr)
		{
			bNeedFixup = true;
			break;
		}
	}

	if (!bNeedFixup)
	{
		for (const UProxyMediaOutput* Proxy : OutputProxies)
		{
			if (Proxy == nullptr)
			{
				bNeedFixup = true;
				break;
			}
		}
	}

	if (bNotConfigured || bNeedFixup)
	{
		FText MessageText = bNotConfigured
			? LOCTEXT("ConfigureText", "The project doesn't have the media proxies configured.")
			: LOCTEXT("FixupText", "The media proxies exist but are not configured properly.");

		TSharedRef<SWidget> Button = SNullWidget::NullWidget;
		if (bNotConfigured)
		{
			SAssignNew(Button, SButton)
			.OnClicked(FOnClicked::CreateRaw(this, &FMediaProfileSettingsCustomization::OnConfigureClicked))
			.Text(LOCTEXT("ConfigureButton", "Configure Now"));
		}

		IDetailCategoryBuilder& MediaProfileCategory = DetailLayout.EditCategory(TEXT("MediaProfile"));
		MediaProfileCategory.AddCustomRow(LOCTEXT("Warning", "Warning"), false)
			.WholeRowWidget
			[
				SNew(SBorder)
				.BorderBackgroundColor(this, &FMediaProfileSettingsCustomization::GetBorderColor)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.LightGroupBorder"))
				.Padding(8.0f)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex(this, &FMediaProfileSettingsCustomization::GetConfigurationStateAsInt)

					+ SWidgetSwitcher::Slot()
					[
						SNew(SHorizontalBox)
						.ToolTipText(MessageText)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("SettingsEditor.WarningIcon"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(16.0f, 0.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FLinearColor::White)
							.ShadowColorAndOpacity(FLinearColor::Black)
							.ShadowOffset(FVector2D::UnitVector)
							.Text(MessageText)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							Button
						]
					]

					+ SWidgetSwitcher::Slot()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("SettingsEditor.GoodIcon"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(16.0f, 0.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FLinearColor::White)
							.ShadowColorAndOpacity(FLinearColor::Black)
							.ShadowOffset(FVector2D::UnitVector)
							.Text(LOCTEXT("MediaProfileValidTooltip", "The media profile proxies are configured."))
						]
					]
				]
			];
	}
}


FReply FMediaProfileSettingsCustomization::OnConfigureClicked()
{
	// Show options window
	TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		if (IMainFrameModule* MainFramePtr = FModuleManager::GetModulePtr<IMainFrameModule>("MainFrame"))
		{
			ParentWindow = MainFramePtr->GetParentWindow();
		}
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("MediaProfileTitle", "Media Profile Configuration"))
		.SizingRule(ESizingRule::Autosized);

	TSharedPtr<SMediaProfileSettingsOptionsWindow> OptionsWindow;
	Window->SetContent
	(
		SAssignNew(OptionsWindow, SMediaProfileSettingsOptionsWindow)
		.WidgetWindow(Window)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	// Configure
	FMediaProfileSettingsCustomizationOptions SettingOptions;
	if (OptionsWindow->ShouldConfigure(SettingOptions))
	{
		Configure(SettingOptions);
		bNotConfigured = false;
		bNeedFixup = false;
	}

	return FReply::Handled();
}


FSlateColor FMediaProfileSettingsCustomization::GetBorderColor() const
{
	if (bNotConfigured)
	{
		return FLinearColor::Red;
	}
	else if (bNeedFixup)
	{
		return FLinearColor::Yellow;
	}
	return FLinearColor::Green;
}


int32 FMediaProfileSettingsCustomization::GetConfigurationStateAsInt() const
{
	return (bNotConfigured || bNeedFixup) ? 0 : 1;
}


namespace MediaProfileSettingsCustomization
{
	template<class T>
	TArray<T*> CreateAssets(int InNumberOfProxies, const TCHAR* InAssetStr, const FString& InPath, TArray<UPackage*>& OutPackagesToSave)
	{
		IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		TArray<T*> Results;

		for (int32 Index = 0; Index < InNumberOfProxies; ++Index)
		{
			FString DesiredName = FString::Printf(TEXT("%s/%s-%02d"), *InPath, InAssetStr, Index + 1);
			FString PackageName;
			FString AssetName;
			AssetTools.CreateUniqueAssetName(DesiredName, TEXT(""), PackageName, AssetName);
			const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
			T* Result = Cast<T>(AssetTools.CreateAsset(AssetName, PackagePath, T::StaticClass(), nullptr));

			Results.Add(Result);
			OutPackagesToSave.Add(Result->GetOutermost());
		}

		return Results;
	}
}


void FMediaProfileSettingsCustomization::Configure(const FMediaProfileSettingsCustomizationOptions& SettingOptions)
{
	if (SettingOptions.IsValid())
	{
		TArray<UPackage*> PackagesToSave;

		FScopedSlowTask Progress(100.0f, LOCTEXT("StartWork", "Configure Media Profile"));
		Progress.MakeDialog(true);

		// Create the sources proxies at the location
		TArray<UProxyMediaSource*> SourceProxies;
		{
			Progress.EnterProgressFrame(25.f);
			SourceProxies = MediaProfileSettingsCustomization::CreateAssets<UProxyMediaSource>(SettingOptions.NumberOfSourceProxies, TEXT("MediaSource"), SettingOptions.ProxiesLocation.Path, PackagesToSave);
			GetMutableDefault<UMediaProfileSettings>()->SetMediaSourceProxy(SourceProxies);
		}

		// Create the output proxies at the location
		TArray<UProxyMediaOutput*> OutputProxies;
		{
			Progress.EnterProgressFrame(25.f);
			OutputProxies = MediaProfileSettingsCustomization::CreateAssets<UProxyMediaOutput>(SettingOptions.NumberOfOutputProxies, TEXT("MediaOutput"), SettingOptions.ProxiesLocation.Path, PackagesToSave);
			GetMutableDefault<UMediaProfileSettings>()->SetMediaOutputProxy(OutputProxies);
		}

		// Create the bundle at the location
		if (SettingOptions.bShouldCreateBundle)
		{
			Progress.EnterProgressFrame(25.f);
			TArray<UMediaBundle*> MediaBundles = MediaProfileSettingsCustomization::CreateAssets<UMediaBundle>(SettingOptions.NumberOfSourceProxies, TEXT("MediaBundle"), SettingOptions.BundlesLocation.Path, PackagesToSave);

			check(MediaBundles.Num() == SourceProxies.Num());
			for (int32 Index = 0; Index < SettingOptions.NumberOfSourceProxies; ++Index)
			{
				UMediaBundle* MediaBundle = MediaBundles[Index];
				UProxyMediaSource* BundleProxy = NewObject<UProxyMediaSource>(MediaBundle);
				BundleProxy->SetMediaSource(SourceProxies[Index]);
				MediaBundle->MediaSource = BundleProxy;
				PackagesToSave.Append(MediaBundle->CreateInternalsEditor());
			}
		}

		// Save the source & output & bundle
		{
			Progress.EnterProgressFrame(25.f);
			bool bOnlyIfIsDirty = false;
			UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, bOnlyIfIsDirty);

			// This will log a warning if the default file was read-only
			const bool bUpdated = GetMutableDefault<UMediaProfileSettings>()->TryUpdateDefaultConfigFile();

			// If the default settings could not be updated, try once to make the default config file writable, then try updating it again
			if (!bUpdated)
			{
				const FString RelativeConfigFilePath = GetMutableDefault<UMediaProfileSettings>()->GetDefaultConfigFilename();
				const FString TargetFilePath = FPaths::ConvertRelativePathToFull(RelativeConfigFilePath);

				constexpr bool bForceSourceControlUpdate = false;
				constexpr bool bShowErrorInNotification = true;
				if (SettingsHelpers::CheckOutOrAddFile(TargetFilePath, bForceSourceControlUpdate, bShowErrorInNotification))
				{
					GetMutableDefault<UMediaProfileSettings>()->TryUpdateDefaultConfigFile();
				}
			}

			// Reapply the media profile if it exist
			UMediaProfile* MediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
			IMediaProfileManager::Get().SetCurrentMediaProfile(nullptr);
			IMediaProfileManager::Get().SetCurrentMediaProfile(MediaProfile);
		}
	}
}

#undef LOCTEXT_NAMESPACE
