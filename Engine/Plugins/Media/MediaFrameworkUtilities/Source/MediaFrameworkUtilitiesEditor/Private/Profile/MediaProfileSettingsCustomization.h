// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FReply;
struct FSlateColor;


struct FMediaProfileSettingsCustomizationOptions;
class IDetailLayoutBuilder;


/**
 *
 */
class FMediaProfileSettingsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	//~ End IDetailCustomization Interface

private:
	// Setup the proxies for the media profile
	FReply OnConfigureClicked();

	FSlateColor GetBorderColor() const;
	int32 GetConfigurationStateAsInt() const;

	void Configure(const FMediaProfileSettingsCustomizationOptions& SettingOptions);

	bool bNotConfigured;
	bool bNeedFixup;
};
