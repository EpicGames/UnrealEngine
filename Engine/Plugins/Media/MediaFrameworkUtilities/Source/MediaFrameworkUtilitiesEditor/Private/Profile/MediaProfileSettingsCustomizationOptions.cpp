// Copyright Epic Games, Inc. All Rights Reserved.

#include "Profile/MediaProfileSettingsCustomizationOptions.h"


FMediaProfileSettingsCustomizationOptions::FMediaProfileSettingsCustomizationOptions()
{
	ProxiesLocation.Path = TEXT("/Game/Media/Proxies/");
	NumberOfSourceProxies = 2;
	NumberOfOutputProxies = 1;
	bShouldCreateBundle = true;
	BundlesLocation.Path = TEXT("/Game/Media/Bundles");
}


bool FMediaProfileSettingsCustomizationOptions::IsValid() const
{
	return !ProxiesLocation.Path.IsEmpty()
		&& (NumberOfSourceProxies > 0 || NumberOfOutputProxies > 0)
		&& (!bShouldCreateBundle || !BundlesLocation.Path.IsEmpty());
}
