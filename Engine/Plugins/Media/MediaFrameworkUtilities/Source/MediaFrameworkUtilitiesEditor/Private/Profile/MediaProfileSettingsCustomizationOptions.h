// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaProfileSettingsCustomizationOptions.generated.h"

/**
 *
 */
USTRUCT()
struct FMediaProfileSettingsCustomizationOptions
{
	GENERATED_BODY()

public:
	FMediaProfileSettingsCustomizationOptions();

	/** The location where the proxies should be created. */
	UPROPERTY(EditAnywhere, Category="Options", meta=(ContentDir))
	FDirectoryPath ProxiesLocation;

	/** The number of input source Unreal may capture. */
	UPROPERTY(EditAnywhere, Category="Options", meta=(UIMin="0", UIMax="8"))
	int32 NumberOfSourceProxies;

	/** The number of output Unreal may generate. */
	UPROPERTY(EditAnywhere, Category="Options", meta=(UIMin="0", UIMax="8"))
	int32 NumberOfOutputProxies;

	/** Create 1 media bundle for every source proxy created. */
	UPROPERTY(EditAnywhere, Category="Options")
	bool bShouldCreateBundle;

	/** The location where the bundles should be created. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (ContentDir, EditCondition="bShouldCreateBundle"))
	FDirectoryPath BundlesLocation;

	bool IsValid() const;
};
