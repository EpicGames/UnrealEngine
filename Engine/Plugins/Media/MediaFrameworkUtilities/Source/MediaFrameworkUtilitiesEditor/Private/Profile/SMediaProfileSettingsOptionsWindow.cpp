// Copyright Epic Games, Inc. All Rights Reserved.

#include "Profile/SMediaProfileSettingsOptionsWindow.h"


#include "DetailsViewArgs.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SWindow.h"


#define LOCTEXT_NAMESPACE "MediaProfileSettingsOptionsWindow"


void SMediaProfileSettingsOptionsWindow::Construct(const FArguments& InArgs)
{
	Window = InArgs._WidgetWindow;
	bConfigure = false;

	TSharedPtr<FStructOnScope> StructScope = MakeShared<FStructOnScope>(FMediaProfileSettingsCustomizationOptions::StaticStruct(), reinterpret_cast<uint8*>(&Options));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowKeyablePropertiesOption = false;
	DetailsViewArgs.bShowAnimatedPropertiesOption = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = "MediaProfileSettings";
	FStructureDetailsViewArgs StructDetailView;
	
	DetailView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructDetailView, StructScope);
	DetailView->SetStructureData(StructScope);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f, 10.0f)
		.AutoHeight()
		[
			DetailView->GetWidget().ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoHeight()
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
			+ SUniformGridPanel::Slot(0, 0)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("Configure", "Configure"))
				.ToolTipText(LOCTEXT("Configure_ToolTip", "Configure the media profile proxies."))
				.OnClicked(this, &SMediaProfileSettingsOptionsWindow::OnConfigure)
				.IsEnabled(MakeAttributeRaw(this, &SMediaProfileSettingsOptionsWindow::CanConfigure))
			]
			+ SUniformGridPanel::Slot(1, 0)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.ToolTipText(LOCTEXT("Cancel_ToolTip", "Cancel the configuration process."))
				.OnClicked(this, &SMediaProfileSettingsOptionsWindow::OnCancel)
			]
		]
	];
}


bool SMediaProfileSettingsOptionsWindow::SupportsKeyboardFocus() const
{
	return true;
}


FReply SMediaProfileSettingsOptionsWindow::OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}


bool SMediaProfileSettingsOptionsWindow::ShouldConfigure(FMediaProfileSettingsCustomizationOptions& OutSettingsOptions) const
{
	OutSettingsOptions = Options;
	return bConfigure;
}


FReply SMediaProfileSettingsOptionsWindow::OnConfigure()
{
	bConfigure = true;

	TSharedPtr<SWindow> SharedWindow = Window.Pin();
	if (SharedWindow.IsValid())
	{
		SharedWindow->RequestDestroyWindow();
	}
	return FReply::Handled();
}


FReply SMediaProfileSettingsOptionsWindow::OnCancel()
{
	bConfigure = false;

	TSharedPtr<SWindow> SharedWindow = Window.Pin();
	if (SharedWindow.IsValid())
	{
		SharedWindow->RequestDestroyWindow();
	}
	return FReply::Handled();
}


bool SMediaProfileSettingsOptionsWindow::CanConfigure() const
{
	return Options.IsValid();
}

#undef LOCTEXT_NAMESPACE
