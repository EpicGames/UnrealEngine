// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Profile/MediaProfileSettingsCustomizationOptions.h"
#include "Widgets/SCompoundWidget.h"


class IStructureDetailsView;
class SWindow;


/*
 *
 */
class SMediaProfileSettingsOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaProfileSettingsOptionsWindow) {}
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	virtual bool SupportsKeyboardFocus() const override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override;

public:
	bool ShouldConfigure(FMediaProfileSettingsCustomizationOptions& OutSettingsOptions) const;

private:
	FReply OnConfigure();
	FReply OnCancel();
	bool CanConfigure() const;

private:
	TWeakPtr<SWindow> Window;
	TSharedPtr<IStructureDetailsView> DetailView;
	FMediaProfileSettingsCustomizationOptions Options;
	bool bConfigure;
};
