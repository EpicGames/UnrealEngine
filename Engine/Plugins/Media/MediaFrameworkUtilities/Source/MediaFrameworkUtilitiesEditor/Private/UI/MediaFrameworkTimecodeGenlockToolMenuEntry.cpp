// Copyright Epic Games, Inc. All Rights Reserved.


#include "MediaFrameworkTimecodeGenlockToolMenuEntry.h"

#include "MediaFrameworkUtilitiesEditorStyle.h"
#include "ToolMenu.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Engine/TimecodeProvider.h"
#include "Misc/Timecode.h"
#include "Profile/MediaProfile.h"
#include "Styling/SlateIconFinder.h"

class UTimecodeProvider;

#define LOCTEXT_NAMESPACE "FMediaFrameworkTimecodeGenlockToolMenuEntry"

FToolMenuEntry FMediaFrameworkTimecodeGenlockToolMenuEntry::CreateTimecodeToolMenuEntry(const TAttribute<bool>& InEntryVisibleAttribute)
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		TEXT("ActiveTimecodeEntry"),
		TAttribute<FText>::CreateSP(this, &FMediaFrameworkTimecodeGenlockToolMenuEntry::GetTimecodeEntryText),
		LOCTEXT("ActiveTimecodeEntryTooltip", "Shows the active timecode"),
		FNewToolMenuDelegate::CreateSP(this, &FMediaFrameworkTimecodeGenlockToolMenuEntry::GetTimecodeDropdownContent),
		false,
		FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), TEXT("ToolbarIcon.Timecode")));

	Entry.Visibility = InEntryVisibleAttribute;
	
	return Entry;
}

FToolMenuEntry FMediaFrameworkTimecodeGenlockToolMenuEntry::CreateGenlockToolMenuEntry(const TAttribute<bool>& InEntryVisibleAttribute)
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		TEXT("ActiveGenlockEntry"),
		TAttribute<FText>(),
		LOCTEXT("ActiveGenlockEntryTooltip", "Shows if there is an active custom time step and if it is currently synchronized"),
		FNewToolMenuDelegate::CreateSP(this, &FMediaFrameworkTimecodeGenlockToolMenuEntry::GetGenlockDropdownContent),
		false,
		TAttribute<FSlateIcon>::CreateSP(this, &FMediaFrameworkTimecodeGenlockToolMenuEntry::GetGenlockEntryIcon));

	Entry.Visibility = InEntryVisibleAttribute;
	
	return Entry;
}

FText FMediaFrameworkTimecodeGenlockToolMenuEntry::GetTimecodeEntryText() const
{
	const FText NullTimecodeText = FText::FromString(FTimecode().ToString());
	
	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return NullTimecodeText;
	}

	UTimecodeProvider* TimecodeProvider = PinnedMediaProfile->GetTimecodeProvider();
	if (!TimecodeProvider)
	{
		return NullTimecodeText;
	}

	const ETimecodeProviderSynchronizationState SyncState = TimecodeProvider->GetSynchronizationState();
	if (SyncState != ETimecodeProviderSynchronizationState::Synchronized && SyncState != ETimecodeProviderSynchronizationState::Synchronizing)
	{
		return NullTimecodeText;
	}

	return FText::FromString(TimecodeProvider->GetTimecode().ToString());
}

FSlateIcon FMediaFrameworkTimecodeGenlockToolMenuEntry::GetGenlockEntryIcon() const
{
	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.WarningWithColor"));
	}

	UEngineCustomTimeStep* CustomTimeStep = PinnedMediaProfile->GetCustomTimeStep();
	if (!CustomTimeStep)
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.WarningWithColor"));
	}

	const ECustomTimeStepSynchronizationState SyncState = CustomTimeStep->GetSynchronizationState();
	if (SyncState != ECustomTimeStepSynchronizationState::Synchronized && SyncState != ECustomTimeStepSynchronizationState::Synchronizing)
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.WarningWithColor"));
	}
	
	return FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), TEXT("ToolbarIcon.Genlock"));
}

void FMediaFrameworkTimecodeGenlockToolMenuEntry::GetTimecodeDropdownContent(UToolMenu* ToolMenu)
{
	FText TimecodeDisplayText = LOCTEXT("TimecodeProviderNotConfiguredText", "Timecode Provider not configured");
	FSlateIcon TimecodeIcon = FSlateIcon();
	
	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (PinnedMediaProfile.IsValid())
	{
		if (UTimecodeProvider* TimecodeProvider = PinnedMediaProfile->GetTimecodeProvider())
		{
			TimecodeDisplayText = TimecodeProvider->GetClass()->GetDisplayNameText();
			TimecodeIcon = FSlateIconFinder::FindIconForClass(TimecodeProvider->GetClass());
		}
	}

	FToolMenuSection& TimecodeProviderSection = ToolMenu->AddSection(TEXT("TimecodeProvider"), LOCTEXT("TimecodeProviderSectionLabel", "Timecode Provider"));
	TimecodeProviderSection.AddMenuEntry("TimecodeProvider",
		TimecodeDisplayText,
		TAttribute<FText>(),
		TimecodeIcon,
		FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([] { return false; })));
}

void FMediaFrameworkTimecodeGenlockToolMenuEntry::GetGenlockDropdownContent(UToolMenu* ToolMenu)
{
	FText GenlockDisplayText = LOCTEXT("GenlockNotConfiguredText", "Genlock not configured");
	FSlateIcon GenlockIcon = FSlateIcon();
	
	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (PinnedMediaProfile.IsValid())
	{
		if (UEngineCustomTimeStep* CustomTimeStep = PinnedMediaProfile->GetCustomTimeStep())
		{
			GenlockDisplayText = FText::FromString(CustomTimeStep->GetDisplayName());
			GenlockIcon = FSlateIconFinder::FindIconForClass(CustomTimeStep->GetClass());
		}
	}

	FToolMenuSection& GenlockSection = ToolMenu->AddSection(TEXT("GenlockSection"), LOCTEXT("GenlockSectionLabel", "Genlock"));
	GenlockSection.AddMenuEntry("Genlock",
		GenlockDisplayText,
		TAttribute<FText>(),
		GenlockIcon,
		FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([] { return false; })));
}

#undef LOCTEXT_NAMESPACE
