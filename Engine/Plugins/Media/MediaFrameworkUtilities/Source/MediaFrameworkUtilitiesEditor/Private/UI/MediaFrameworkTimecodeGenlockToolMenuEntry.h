// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolMenuEntry.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMediaProfile;

/**
 * Creates and manages tool menu entries for timecode and genlock configurations on media profiles
 */
class FMediaFrameworkTimecodeGenlockToolMenuEntry: public TSharedFromThis<FMediaFrameworkTimecodeGenlockToolMenuEntry>
{
public:
	FMediaFrameworkTimecodeGenlockToolMenuEntry(TWeakObjectPtr<UMediaProfile> InMediaProfile)
		: MediaProfile(InMediaProfile)
	{ }
	
	/** Creates the timecode tool menu entry that can be added to toolbars  */
	FToolMenuEntry CreateTimecodeToolMenuEntry(const TAttribute<bool>& InEntryVisibleAttribute);

	/** Creates the genlock tool menu entry that can be added to toolbars  */
	FToolMenuEntry CreateGenlockToolMenuEntry(const TAttribute<bool>& InEntryVisibleAttribute);

private:
	/** Gets the text for the timecode toolbar entry */
	FText GetTimecodeEntryText() const;

	/** Gets the icon for the genlock toolbar entry */
	FSlateIcon GetGenlockEntryIcon() const;

	/** Fills the dropdown menu for the timecode toolbar entry */
	void GetTimecodeDropdownContent(UToolMenu* ToolMenu);

	/** Fills the dropdown menu for the genlock toolbar entry */
	void GetGenlockDropdownContent(UToolMenu* ToolMenu);
	
private:
	/** Weak pointer to the media profile to source the timecode and genlock configuration from */
	TWeakObjectPtr<UMediaProfile> MediaProfile;
};
