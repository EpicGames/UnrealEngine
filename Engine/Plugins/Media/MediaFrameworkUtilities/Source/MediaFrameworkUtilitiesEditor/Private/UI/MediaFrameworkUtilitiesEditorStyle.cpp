// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/MediaFrameworkUtilitiesEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir MediaFrameworkUtilitiesStyle::StyleInstance->RootToContentDir
#define RootToCoreContentDir MediaFrameworkUtilitiesStyle::StyleInstance->RootToCoreContentDir

namespace MediaFrameworkUtilitiesStyle
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FName NAME_StyleName(TEXT("MediaBundleStyle"));
	const FName NAME_ContextName(TEXT("MediaBundle"));

	static TUniquePtr<FSlateStyleSet> StyleInstance;
}

void FMediaFrameworkUtilitiesEditorStyle::Register()
{
	MediaFrameworkUtilitiesStyle::StyleInstance = MakeUnique<FSlateStyleSet>(MediaFrameworkUtilitiesStyle::NAME_StyleName);
	MediaFrameworkUtilitiesStyle::StyleInstance->SetCoreContentRoot(FPaths::EngineContentDir());
	MediaFrameworkUtilitiesStyle::StyleInstance->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Media/MediaFrameworkUtilities/Content/Editor/Icons/"));

	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ClassThumbnail.MediaBundle", new IMAGE_BRUSH("MediaBundle_64x", MediaFrameworkUtilitiesStyle::Icon64x64));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ClassIcon.MediaBundle", new IMAGE_BRUSH("MediaBundle_20x", MediaFrameworkUtilitiesStyle::Icon20x20));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ClassThumbnail.ProxyMediaOutput", new IMAGE_BRUSH("ProxyMediaOutput_64x", MediaFrameworkUtilitiesStyle::Icon64x64));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ClassIcon.ProxyMediaOutput", new IMAGE_BRUSH("ProxyMediaOutput_16x", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ClassThumbnail.ProxyMediaSource", new IMAGE_BRUSH("ProxyMediaSource_64x", MediaFrameworkUtilitiesStyle::Icon64x64));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ClassIcon.ProxyMediaSource", new IMAGE_BRUSH("ProxyMediaSource_16x", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ClassThumbnail.MediaProfile", new IMAGE_BRUSH("MediaProfile_64x", MediaFrameworkUtilitiesStyle::Icon64x64));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ClassIcon.MediaProfile", new IMAGE_BRUSH_SVG("MediaProfile", MediaFrameworkUtilitiesStyle::Icon20x20));

	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.Edit", new IMAGE_BRUSH_SVG("MediaProfile", MediaFrameworkUtilitiesStyle::Icon40x40));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.Apply", new IMAGE_BRUSH("Apply_40x", MediaFrameworkUtilitiesStyle::Icon40x40));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.ClearCurrentMediaProfile", new IMAGE_BRUSH_SVG("MediaProfile", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.SaveLayout", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Starship/Common/LayoutSave", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.SaveLayoutAs", new CORE_IMAGE_BRUSH_SVG( "Editor/Slate/Starship/Common/SaveCurrentAs", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.LoadLayout", new CORE_IMAGE_BRUSH_SVG( "Editor/Slate/Starship/Common/LayoutLoad", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.RemoveLayout", new CORE_IMAGE_BRUSH_SVG( "Editor/Slate/Starship/Common/LayoutRemove", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.ToggleFullscreen", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Starship/Common/EnableFullscreen", MediaFrameworkUtilitiesStyle::Icon16x16));

	MediaFrameworkUtilitiesStyle::StyleInstance->Set("TabIcons.MediaCapture.Small", new IMAGE_BRUSH_SVG("MediaCapture", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("MediaCapture.Capture", new IMAGE_BRUSH("PlayControlsRecord", MediaFrameworkUtilitiesStyle::Icon20x20));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("MediaCapture.Stop", new IMAGE_BRUSH("CaptureCameraViewport_Stop_40x", MediaFrameworkUtilitiesStyle::Icon40x40));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("MediaCapture.Settings", new IMAGE_BRUSH("Settings_40x", MediaFrameworkUtilitiesStyle::Icon40x40));

	MediaFrameworkUtilitiesStyle::StyleInstance->Set("TabIcons.VideoInput.Small", new IMAGE_BRUSH_SVG("MediaPlayback", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("VideoInput.Play", new IMAGE_BRUSH("MediaSource_Play_40x", MediaFrameworkUtilitiesStyle::Icon40x40));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("VideoInput.Stop", new IMAGE_BRUSH("CaptureCameraViewport_Stop_40x", MediaFrameworkUtilitiesStyle::Icon40x40));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("VideoInput.Settings", new IMAGE_BRUSH("Settings_40x", MediaFrameworkUtilitiesStyle::Icon40x40));

	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.Timecode", new IMAGE_BRUSH_SVG("Timecode", MediaFrameworkUtilitiesStyle::Icon16x16));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.Genlock", new IMAGE_BRUSH_SVG("Genlock", MediaFrameworkUtilitiesStyle::Icon16x16));

	MediaFrameworkUtilitiesStyle::StyleInstance->Set("ToolbarIcon.Refresh", new CORE_IMAGE_BRUSH_SVG("Slate/Starship/Common/Update", MediaFrameworkUtilitiesStyle::Icon16x16));
	
	//Use existing icon of MaterialInstanceConstant from Engine Editor content
	const FString EngineContentPath = FPaths::EngineContentDir() / TEXT("Editor/Slate/Icons/AssetIcons/MaterialInstanceConstant_64x.png");
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("MaterialEditor", new FSlateImageBrush(EngineContentPath, MediaFrameworkUtilitiesStyle::Icon40x40));
	MediaFrameworkUtilitiesStyle::StyleInstance->Set("MaterialEditor.Small", new FSlateImageBrush(EngineContentPath, MediaFrameworkUtilitiesStyle::Icon20x20));


	FSlateStyleRegistry::RegisterSlateStyle(*MediaFrameworkUtilitiesStyle::StyleInstance.Get());
}

void FMediaFrameworkUtilitiesEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*MediaFrameworkUtilitiesStyle::StyleInstance.Get());
	MediaFrameworkUtilitiesStyle::StyleInstance.Reset();
}

FName FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName()
{
	return MediaFrameworkUtilitiesStyle::NAME_StyleName;
}

const ISlateStyle& FMediaFrameworkUtilitiesEditorStyle::Get()
{
	check(MediaFrameworkUtilitiesStyle::StyleInstance.IsValid());
	return *MediaFrameworkUtilitiesStyle::StyleInstance.Get();
}

#undef IMAGE_BRUSH
#undef RootToContentDir
#undef RootToCoreContentDir