// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/MediaProfileMenuEntry.h"

#include "AssetEditor/MediaProfileCommands.h"
#include "AssetToolsModule.h"
#include "ClassViewerFilter.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "IAssetTools.h"
#include "MediaOutput.h"
#include "MediaProfileMenus.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfileSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "AssetEditor/MediaProfileEditorUserSettings.h"

#define LOCTEXT_NAMESPACE "MediaProfileEditor"

struct FMediaProfileMenuEntryImpl
{
	FMediaProfileMenuEntryImpl()
	{
		TSharedPtr<FUICommandList> Actions = MakeShared<FUICommandList>();

		// Action to edit the current selected media profile
		Actions->MapAction(FMediaProfileCommands::Get().Edit, FExecuteAction::CreateStatic(&UE::MediaProfile::Menus::OpenExistingOrCreateNewMediaProfile));


		FString ButtonAlignmentString;

		/** 
		 * These settings aren't meant to be user-configurable, they're meant to allow an application (Currently LiveLinkHub)
		 * to provide an override to the default location and alignment of the widget. This ultimately allows hosting the MediaProfile toolbar button
		 * in a different asset editor than the Level Editor.
		 */
		GConfig->GetString(TEXT("MediaProfile"), TEXT("ButtonMenu"), ToolbarMenu, GEngineIni);
		GConfig->GetString(TEXT("MediaProfile"), TEXT("ButtonAlignment"), ButtonAlignmentString, GEngineIni);

		if (ToolbarMenu.IsEmpty())
		{
			ToolbarMenu = TEXT("LevelEditor.LevelEditorToolBar.User");
		}

		EToolMenuSectionAlign ButtonAlignment = EToolMenuSectionAlign::Default;
		if (!ButtonAlignmentString.IsEmpty())
		{
			ButtonAlignment = static_cast<EToolMenuSectionAlign>(StaticEnum<EToolMenuSectionAlign>()->GetValueByName(*ButtonAlignmentString));
		}

		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(*ToolbarMenu);
		FToolMenuSection& Section = Menu->FindOrAddSection("MediaProfile");
		Section.Alignment = ButtonAlignment;

		auto ButtonTooltipLambda = [this]() -> FText
		{
			bool bUseTransientProfile = false;
			GConfig->GetBool(TEXT("MediaProfile"), TEXT("bUseTransientProfile"), bUseTransientProfile, GEngineIni);

			// In transient mode (e.g. LLH) always show a generic tooltip since the
			// internal object name (MediaProfile_0) is meaningless to the user.
			if (bUseTransientProfile)
			{
				return LOCTEXT("TransientMediaProfile_ToolTip", "Open the Media Profile editor");
			}

			UMediaProfile* MediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
			if (MediaProfile == nullptr)
			{
				return LOCTEXT("EmptyMediaProfile_ToolTip", "Create a new media profile");
			}
			return FText::Format(LOCTEXT("MediaProfile_ToolTip", "Edit '{0}'")
				, FText::FromName(MediaProfile->GetFName()));
		};

		// Add a button to edit the current media profile
		FToolMenuEntry MediaProfileButtonEntry = FToolMenuEntry::InitToolBarButton(
			FMediaProfileCommands::Get().Edit,
			LOCTEXT("MediaProfile_Label", "Media Profile"),
			MakeAttributeLambda(ButtonTooltipLambda),
			FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), TEXT("ToolbarIcon.Edit"))
		);
		MediaProfileButtonEntry.SetCommandList(Actions);
		MediaProfileButtonEntry.ToolBarData.LabelOverride = FText::GetEmpty();

		FToolMenuEntry MediaProfileMenuEntry = FToolMenuEntry::InitComboButton(
		"MediaProfileMenu",
		FUIAction(),
		FOnGetContent::CreateStatic(&UE::MediaProfile::Menus::GenerateMediaProfileDropdownMenu),
		LOCTEXT("LevelEditorToolbarMediaProfileButtonLabel", "MediaProfile"),
		LOCTEXT("LevelEditorToolbarMediaProfileButtonTooltip", "Configure current MediaProfile"),
		FSlateIcon(),
		true //bInSimpleComboBox
		);

		MediaProfileMenuEntry.StyleNameOverride = "CalloutToolbar";
		Section.AddEntry(MediaProfileButtonEntry);
		Section.AddEntry(MediaProfileMenuEntry);
	}

	~FMediaProfileMenuEntryImpl()
	{
		if (!IsEngineExitRequested() && UObjectInitialized())
		{
			UToolMenus::Get()->RemoveSection(*ToolbarMenu, "MediaProfile");
		}
	}

public:
	/** Path of the toolbar that should host the MediaProfile toolbar button. If unspecified by the config, it will default to the Level Editor's toolbar. */
	FString ToolbarMenu;
	static TUniquePtr<FMediaProfileMenuEntryImpl> Implementation;
};

TUniquePtr<FMediaProfileMenuEntryImpl> FMediaProfileMenuEntryImpl::Implementation;

void FMediaProfileMenuEntry::Register()
{
	if (!IsRunningCommandlet() && GetDefault<UMediaProfileEditorSettings>()->bDisplayInToolbar)
	{
		FMediaProfileMenuEntryImpl::Implementation = MakeUnique<FMediaProfileMenuEntryImpl>();
	}
}

void FMediaProfileMenuEntry::Unregister()
{
	FMediaProfileMenuEntryImpl::Implementation.Reset();
}

#undef LOCTEXT_NAMESPACE
