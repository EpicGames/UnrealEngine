// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaFrameworkTimecodeGenlockHeader.h"

#include "Components/HorizontalBox.h"
#include "Engine/Engine.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Engine/TimecodeProvider.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaFrameworkTimecodeGenlockHeader"

void SMediaFrameworkTimecodeGenlockHeader::Construct(const FArguments& InArgs)
{
	ShowTimecodeAttr = InArgs._ShowTimecode;
	ShowGenlockAttr = InArgs._ShowGenlock;
	OnOpenTimecodeTab = InArgs._OnOpenTimecodeTab;

	TSharedRef<SWidget> ButtonContent = SNew(SHorizontalBox)
		.ToolTipText(this, &SMediaFrameworkTimecodeGenlockHeader::GetTooltipText)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(8.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMediaFrameworkTimecodeGenlockHeader::GetTimecodeText)
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
			.Visibility(this, &SMediaFrameworkTimecodeGenlockHeader::GetElementVisibility, true, false)
		]
		
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
			.Thickness(2)
			.Visibility(this, &SMediaFrameworkTimecodeGenlockHeader::GetElementVisibility, true, true)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(8.0)
		[
			SNew(SImage)
			.Image(this, &SMediaFrameworkTimecodeGenlockHeader::GetGenlockIcon)
			.DesiredSizeOverride(FVector2D(20, 20))
			.Visibility(this, &SMediaFrameworkTimecodeGenlockHeader::GetElementVisibility, false, true)
		]
		
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(8.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMediaFrameworkTimecodeGenlockHeader::GetGenlockText)
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
			.Visibility(this, &SMediaFrameworkTimecodeGenlockHeader::GetElementVisibility, false, true)
		];

	TSharedRef<SWidget> ChildContent = ButtonContent;
	if (InArgs._IsButton)
	{
		ChildContent = SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ContentPadding(0.0f)
			.OnClicked(this, &SMediaFrameworkTimecodeGenlockHeader::OnButtonClicked)
			[
				ButtonContent
			];
	}
	
	ChildSlot
	[
		ChildContent
	];
}

FText SMediaFrameworkTimecodeGenlockHeader::GetTooltipText() const
{
	UTimecodeProvider* TimecodeProvider = GEngine->GetTimecodeProvider();
	if (!TimecodeProvider)
	{
		return LOCTEXT("NoTimecodeProviderTextFormat", "Timecode not configured");
	}
	
	FText SourceName = LOCTEXT("EngineSourceText", "Engine");
	FText SourceType = LOCTEXT("EngineSourceTypeText", "Engine");
	
	if (UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile())
	{
		if (ActiveMediaProfile->GetTimecodeProvider() == TimecodeProvider)
		{
			SourceName = FText::FromString(ActiveMediaProfile->GetName());
			SourceType = LOCTEXT("MediaProfileTypeText", "Media Profile");
		}
	}
	
	const FText TimecodeProviderName = TimecodeProvider->GetClass()->GetDisplayNameText();
	return FText::Format(LOCTEXT("TooltipTextFormat", "Timecode Provider: {0}\nSet by: {1} ({2})"), TimecodeProviderName, SourceName, SourceType);
}

FText SMediaFrameworkTimecodeGenlockHeader::GetTimecodeText() const
{
	const FText NullTimecodeText = FText::FromString(FTimecode().ToString());

	UTimecodeProvider* TimecodeProvider = GEngine->GetTimecodeProvider();
	if (!TimecodeProvider)
	{
		return NullTimecodeText;
	}

	const ETimecodeProviderSynchronizationState SyncState = TimecodeProvider->GetSynchronizationState();
	if (SyncState != ETimecodeProviderSynchronizationState::Synchronized && SyncState != ETimecodeProviderSynchronizationState::Synchronizing)
	{
		return NullTimecodeText;
	}
 
	return FText::FromString(TimecodeProvider->GetTimecode().ToString());
}

const FSlateBrush* SMediaFrameworkTimecodeGenlockHeader::GetGenlockIcon() const
{
	const FSlateBrush* WarningIcon = FAppStyle::GetBrush("Icons.WarningWithColor");
	
	UEngineCustomTimeStep* CustomTimeStep = GEngine->GetCustomTimeStep();
	if (!CustomTimeStep)
	{
		return WarningIcon;
	}

	const ECustomTimeStepSynchronizationState SyncState = CustomTimeStep->GetSynchronizationState();
	if (SyncState != ECustomTimeStepSynchronizationState::Synchronized && SyncState != ECustomTimeStepSynchronizationState::Synchronizing)
	{
		return WarningIcon;
	}
 
	return nullptr;
}

FText SMediaFrameworkTimecodeGenlockHeader::GetGenlockText() const
{
	UEngineCustomTimeStep* CustomTimeStep = GEngine->GetCustomTimeStep();
	if (!CustomTimeStep)
	{
		return LOCTEXT("NoCustomTimeStepLabel", "No custom time step configured");
	}
 
	return FText::FromString(CustomTimeStep->GetDisplayName());
}

FReply SMediaFrameworkTimecodeGenlockHeader::OnButtonClicked()
{
	OnOpenTimecodeTab.ExecuteIfBound();
	return FReply::Handled();
}

EVisibility SMediaFrameworkTimecodeGenlockHeader::GetElementVisibility(bool bTimecode, bool bGenlock) const
{
	const bool bShowTimecode = ShowTimecodeAttr.Get(true);
	const bool bShowGenlock = ShowGenlockAttr.Get(true);
	
	if (bTimecode && bGenlock)
	{
		return (bShowTimecode && bShowGenlock) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	if (bTimecode)
	{
		return bShowTimecode ? EVisibility::Visible : EVisibility::Collapsed;
	}
	if (bGenlock)
	{
		return bShowGenlock ? EVisibility::Visible : EVisibility::Collapsed;
	}
	
	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE