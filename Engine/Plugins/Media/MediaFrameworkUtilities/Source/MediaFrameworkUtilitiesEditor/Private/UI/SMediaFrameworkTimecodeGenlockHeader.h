// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UMediaProfile;

/**
 * Widget that displays the timecode and genlock status for a media profile that can be displayed in toolbars or details panel headers
 */
class SMediaFrameworkTimecodeGenlockHeader : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMediaFrameworkTimecodeGenlockHeader)
		: _IsButton(false)
		, _ShowTimecode(true)
		, _ShowGenlock(true)
	{ }
		SLATE_ARGUMENT(bool, IsButton)
		SLATE_ATTRIBUTE(bool, ShowTimecode)
		SLATE_ATTRIBUTE(bool, ShowGenlock)
		SLATE_EVENT(FSimpleDelegate, OnOpenTimecodeTab)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

private:
	/** Gets the text to display in the tooltip for the widget */
	FText GetTooltipText() const;

	/** Gets the text to display for the timecode */
	FText GetTimecodeText() const;

	/** Gets the icon to display for the genlock */
	const FSlateBrush* GetGenlockIcon() const;

	/** Gets the text to display for the genlock */
	FText GetGenlockText() const;

	/** Raised when the toolbar button is clicked */
	FReply OnButtonClicked();

	/** Gets whether certain elements of the toolbar entry should be visible */
	EVisibility GetElementVisibility(bool bTimecode, bool bGenlock) const;
	
private:
	/** Gets whether the timecode should be shown */
	TAttribute<bool> ShowTimecodeAttr;

	/** Gets whether the genlock should be shown */
	TAttribute<bool> ShowGenlockAttr;
	
	/** Raised when the header is requesting the TC/Genlock tab be opened/focused */
	FSimpleDelegate OnOpenTimecodeTab;
};
