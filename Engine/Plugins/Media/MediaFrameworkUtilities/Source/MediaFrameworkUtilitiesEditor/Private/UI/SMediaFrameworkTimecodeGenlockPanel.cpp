// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMediaFrameworkTimecodeGenlockPanel.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailCustomization.h"
#include "MediaFrameworkUtilitiesEditorStyle.h"
#include "SMediaFrameworkTimecodeGenlockHeader.h"
#include "SPositiveActionButton.h"
#include "AssetEditor/MediaProfileEditorUserSettings.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Engine/TimecodeProvider.h"
#include "Misc/Timecode.h"
#include "Modules/ModuleManager.h"
#include "Profile/MediaProfile.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMediaFrameworkTimecodeGenlockPanel"

/** Details panel customization to hide any categories unrelated to timecode or genlock within the media profile  */
class FMediaProfileTimecodeGenlockDetailsCustomization : public IDetailCustomization
{
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		DetailBuilder.HideCategory(TEXT("Inputs"));
		DetailBuilder.HideCategory(TEXT("Outputs"));

		IDetailCategoryBuilder& TimecodeCategory = DetailBuilder.EditCategory(TEXT("Timecode Provider"));

		TimecodeCategory.AddCustomRow(LOCTEXT("ShowTimecodeFilterString", "Show Timecode"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ShowTimecodeLabel", "Show Timecode in Toolbar"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([]()
			{
				if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
				{
					return UserSettings->bShowTimecodeInEditorToolbar ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}

				return ECheckBoxState::Checked;
			})
			.OnCheckStateChanged_Lambda([](ECheckBoxState CheckBoxState)
			{
				if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
				{
					UserSettings->bShowTimecodeInEditorToolbar = CheckBoxState == ECheckBoxState::Checked;
				}
			})
		];

		IDetailCategoryBuilder& GenlockCategory = DetailBuilder.EditCategory(TEXT("Genlock"));

		GenlockCategory.AddCustomRow(LOCTEXT("ShowGenlockFilterString", "Show Genlock"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ShowGenlockLabel", "Show Genlock in Toolbar"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([]()
			{
				if (const UMediaProfileEditorUserSettings* UserSettings = GetDefault<UMediaProfileEditorUserSettings>())
				{
					return UserSettings->bShowGenlockInEditorToolbar ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}

				return ECheckBoxState::Checked;
			})
			.OnCheckStateChanged_Lambda([](ECheckBoxState CheckBoxState)
			{
				if (UMediaProfileEditorUserSettings* UserSettings = GetMutableDefault<UMediaProfileEditorUserSettings>())
				{
					UserSettings->bShowGenlockInEditorToolbar = CheckBoxState == ECheckBoxState::Checked;
				}
			})
		];
	}
};

void SMediaFrameworkTimecodeGenlockPanel::Construct(const FArguments& InArgs)
{
	MediaProfile = InArgs._MediaProfile;
	
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsArgs;
	DetailsArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsArgs.bCustomNameAreaLocation = true;
	DetailsArgs.bCustomFilterAreaLocation = true;
	DetailsArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsArgs.bShowSectionSelector = true;
	DetailsArgs.NotifyHook = this;
	
	DetailsView = PropertyModule.CreateDetailView(DetailsArgs);

	DetailsView->RegisterInstancedCustomPropertyLayout(UMediaProfile::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([this]
	{
		return MakeShared<FMediaProfileTimecodeGenlockDetailsCustomization>();
	}));

	if (MediaProfile.IsValid())
	{
		DetailsView->SetObject(MediaProfile.Get());
	}
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SMediaFrameworkTimecodeGenlockHeader)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Horizontal)
			.Thickness(2)
		]
		
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			DetailsView->GetFilterAreaWidget().ToSharedRef()
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(6.0f)
		[
			SNew(SPositiveActionButton)
			.Icon(FMediaFrameworkUtilitiesEditorStyle::Get().GetBrush("ToolbarIcon.Refresh"))
			.Text(LOCTEXT("RefreshButtonLabel", "Refresh Timecode/Genlock"))
			.ToolTipText(LOCTEXT("RefreshButtonTooltip", "Refreshes the currently active timecode/genlock to match the settings"))
			.Visibility_Lambda([this]
			{
				return MediaProfile.IsValid() ? EVisibility::Visible : EVisibility::Hidden;
			})
			.OnClicked_Lambda([this]
			{
				constexpr bool bRefreshTimecode = true;
				constexpr bool bRefreshGenlock = true;
				RefreshTimecodeGenlock(bRefreshTimecode, bRefreshGenlock);

				return FReply::Handled();
			})
		]

		+SVerticalBox::Slot()
		[
			DetailsView.ToSharedRef()
		]
	];
}

void SMediaFrameworkTimecodeGenlockPanel::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged)
{
	FProperty* HeadProperty = PropertyThatChanged->GetHead()->GetValue();
	if (!HeadProperty)
	{
		return;
	}
	
	bool bRefreshTimecode = false;
	if (HeadProperty->GetFName() == TEXT("TimecodeProvider") || HeadProperty->GetFName() == TEXT("bOverrideTimecodeProvider"))
	{
		bRefreshTimecode = true;
	}

	bool bRefreshGenlock = false;
	if (HeadProperty->GetFName() == TEXT("CustomTimeStep") || HeadProperty->GetFName() == TEXT("bOverrideCustomTimeStep"))
	{
		bRefreshGenlock = true;
	}

	RefreshTimecodeGenlock(bRefreshTimecode, bRefreshGenlock);
}

void SMediaFrameworkTimecodeGenlockPanel::RefreshTimecodeGenlock(bool bRefreshTimecode, bool bRefreshGenlock)
{
	TStrongObjectPtr<UMediaProfile> PinnedMediaProfile = MediaProfile.Pin();
	if (!PinnedMediaProfile.IsValid())
	{
		return;
	}
	
	if (bRefreshTimecode)
	{
		PinnedMediaProfile->ApplyTimecodeProvider();
	}

	if (bRefreshGenlock)
	{
		PinnedMediaProfile->ApplyCustomTimeStep();
	}
}

#undef LOCTEXT_NAMESPACE
