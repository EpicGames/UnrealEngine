// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

class IDetailsView;
class UMediaProfile;

/**
 * Details panel that displays a media profile's Timecode and Genlock settings, as well as
 * a timecode/genlock status bar and optionally a media profile selection dropdown
 */
class SMediaFrameworkTimecodeGenlockPanel : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SMediaFrameworkTimecodeGenlockPanel) { }
		SLATE_ARGUMENT(TWeakObjectPtr<UMediaProfile>, MediaProfile)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

	//~ Begin FNotifyHook Interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged) override;
	//~ End FNotifyHook Interface

private:
	/** Refreshes the timecode and genlock to match the settings in the current media profile */
	void RefreshTimecodeGenlock(bool bRefreshTimecode, bool bRefreshGenlock);

private:
	/** Media profile whose timecode/genlock settings are being displayed */
	TWeakObjectPtr<UMediaProfile> MediaProfile;

	/** Details view displaying timecode/genlock properties */
	TSharedPtr<IDetailsView> DetailsView;
};
