// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoInputTab/SMediaFrameworkVideoInput.h"

#include "DetailsViewArgs.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "LevelEditor.h"
#include "MediaBundle.h"
#include "MediaFrameworkVideoInputSettings.h"
#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlateOptMacros.h"
#include "UI/MediaFrameworkUtilitiesEditorStyle.h"
#include "VideoInputTab/SMediaFrameworkVideoInputDisplay.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"


#define LOCTEXT_NAMESPACE "MediaFrameworkVideoInput"

namespace MediaFrameworkVideoInputUtils
{
	static const FName MediaFrameworkUtilitiesApp = FName("MediaFrameworkVideoInputApp");
	static const FName LevelEditorModuleName("LevelEditor");

	TSharedRef<SDockTab> CreateMediaFrameworkVideoInputTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SMediaFrameworkVideoInput)
			];
	}

	UMediaFrameworkVideoInputSettings* GetVideoInputSettings()
	{
		return GetMutableDefault<UMediaFrameworkVideoInputSettings>();
	}

	/**
	 * Verify if there is one object that the settings depends on.
	 * @return true if there is a dependence
	 */
	bool AreSettingsDependentOn(const TArray<UObject*>& Objects)
	{
		UMediaFrameworkVideoInputSettings* UserSettings = GetVideoInputSettings();

		for (TSoftObjectPtr<UMediaBundle>& SoftMediaBundlePtr : UserSettings->MediaBundles)
		{
			bool bSettingDependantOnObject = true;

			UMediaBundle* MediaBundle = SoftMediaBundlePtr.LoadSynchronous();
			if (MediaBundle &&
				!Objects.Contains(MediaBundle) &&
				!Objects.Contains(MediaBundle->GetMediaPlayer()) &&
				!Objects.Contains(MediaBundle->GetMediaTexture()) &&
				!Objects.Contains(MediaBundle->GetMediaSource()))
			{
				bSettingDependantOnObject = false;
			}

			if (bSettingDependantOnObject)
			{
				return true;
			}
		}
		for (const FMediaFrameworkVideoInputSourceSettings& Media : UserSettings->MediaSources)
		{
			bool bSettingDependantOnObject = true;

			UMediaSource* MediaSource = Media.MediaSource.LoadSynchronous();
			if (MediaSource && !Objects.Contains(MediaSource))
			{
				bSettingDependantOnObject = false;
			}

			UMediaTexture* MediaTexture = Media.MediaTexture.LoadSynchronous();
			if (MediaTexture
				&& !Objects.Contains(MediaTexture)
				&& !Objects.Contains(MediaTexture->GetMediaPlayer()))
			{
				bSettingDependantOnObject = false;
			}

			if (bSettingDependantOnObject)
			{
				return true;
			}
		}

		return false;
	}

	const float PaddingTopForViewportBox = 4.f;
	/*
	 * SVideoInputDisplayVerticalBox looks like a SVerticalBox, but it's specialize for the display of a video input from MediaSources and MediaBundles
	 */
	class SVideoInputDisplayVerticalBox : public SVerticalBox
	{
	public:

		virtual ~SVideoInputDisplayVerticalBox()
		{
			Empty();
		}

		void DisplayVideoInput(UMediaBundle* InMediaBundle)
		{
			check(InMediaBundle);

			TSharedRef<SMediaFrameworkVideoInputDisplay> VideoInputDisplay =
				SNew(SMediaFrameworkVideoInputMediaBundleDisplay)
				.MediaBundle(MakeWeakObjectPtr(InMediaBundle));

			Add(VideoInputDisplay);
		}

		void DisplayVideoInput(UMediaSource* InMediaSource, UMediaTexture* InMediaTexture)
		{
			check(InMediaSource);

			TSharedRef<SMediaFrameworkVideoInputDisplay> VideoInputDisplay =
				SNew(SMediaFrameworkVideoInputMediaSourceDisplay)
				.MediaSource(MakeWeakObjectPtr(InMediaSource))
				.MediaTexture(MakeWeakObjectPtr(InMediaTexture));

			Add(VideoInputDisplay);
		}

		void Empty()
		{
			//remove the ui
			for (const TSharedRef<SMediaFrameworkVideoInputDisplay>& VideoInputDisplay : VideoInputDisplays)
			{
				RemoveSlot(VideoInputDisplay);
			}
			VideoInputDisplays.Empty();
		}

	private:
		void Add(TSharedRef<SMediaFrameworkVideoInputDisplay>& InVideoInputViewport)
		{
			AddSlot()
			.Padding(0.f, PaddingTopForViewportBox, 0.f, 0.f)
			[
				InVideoInputViewport
			];

			VideoInputDisplays.Add(InVideoInputViewport);
		}

		TArray<TSharedRef<SMediaFrameworkVideoInputDisplay>> VideoInputDisplays;
	};

}

FDelegateHandle SMediaFrameworkVideoInput::LevelEditorTabManagerChangedHandle;

void SMediaFrameworkVideoInput::RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem)
{
	auto RegisterTabSpawner = [InWorkspaceItem]()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(MediaFrameworkVideoInputUtils::LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		LevelEditorTabManager->RegisterTabSpawner(MediaFrameworkVideoInputUtils::MediaFrameworkUtilitiesApp, FOnSpawnTab::CreateStatic(&MediaFrameworkVideoInputUtils::CreateMediaFrameworkVideoInputTab))
			.SetDisplayName(LOCTEXT("TabTitle", "Media Playback"))
			.SetTooltipText(LOCTEXT("TabTooltipText", "Tool to open diverse video sources."))
			.SetGroup(InWorkspaceItem)
			.SetIcon(FSlateIcon(FMediaFrameworkUtilitiesEditorStyle::GetStyleSetName(), "TabIcons.VideoInput.Small"));
	};

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(MediaFrameworkVideoInputUtils::LevelEditorModuleName);
	if (LevelEditorModule.GetLevelEditorTabManager())
	{
		RegisterTabSpawner();
	}
	else
	{
		LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterTabSpawner);
	}
}

void SMediaFrameworkVideoInput::UnregisterNomadTabSpawner()
{
	if (FSlateApplication::IsInitialized() && FModuleManager::Get().IsModuleLoaded(MediaFrameworkVideoInputUtils::LevelEditorModuleName))
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(MediaFrameworkVideoInputUtils::LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager;
		if (LevelEditorModule)
		{
			LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager();
			LevelEditorModule->OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
		}

		if (LevelEditorTabManager.IsValid())
		{
			LevelEditorTabManager->UnregisterTabSpawner(MediaFrameworkVideoInputUtils::MediaFrameworkUtilitiesApp);
		}
	}
}

SMediaFrameworkVideoInput::~SMediaFrameworkVideoInput()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.RemoveAll(this);
	FEditorDelegates::OnAssetsPreDelete.RemoveAll(this);
	Stop();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMediaFrameworkVideoInput::Construct(const FArguments& InArgs)
{
	bIsPlaying = false;

	FEditorDelegates::OnAssetsPreDelete.AddRaw(this, &SMediaFrameworkVideoInput::OnAssetsPreDelete);
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddRaw(this, &SMediaFrameworkVideoInput::OnObjectPreEditChange);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &SMediaFrameworkVideoInput::OnObjectPostEditChange);

	UMediaFrameworkVideoInputSettings* UserSettings = MediaFrameworkVideoInputUtils::GetVideoInputSettings();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = "MediaFrameworkUtilitites";
	DetailView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView->SetObject(UserSettings);


	VideosViewport = SNew(MediaFrameworkVideoInputUtils::SVideoInputDisplayVerticalBox);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(2.f))
		[
			MakeToolBar()
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(FMargin(2.f))
		[
			SAssignNew(Splitter, SSplitter)
			.Orientation(MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bIsVerticalSplitterOrientation ? EOrientation::Orient_Vertical : EOrientation::Orient_Horizontal)
			+ SSplitter::Slot()
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.IsEnabled_Lambda([this]() { return !IsPlaying(); })
				[
					DetailView.ToSharedRef()
				]
			]
			+ SSplitter::Slot()
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						VideosViewport.ToSharedRef()
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<class SWidget> SMediaFrameworkVideoInput::MakeToolBar()
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
	ToolBarBuilder.BeginSection(TEXT("Player"));
	{
		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					Play();
				}),
				FCanExecuteAction::CreateLambda([this]
				{
					return CanPlay() && !bIsPlaying;
				})),
			NAME_None,
			LOCTEXT("Play_Label", "Play"),
			LOCTEXT("Play_ToolTip", "Open the video feeds"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Play")
			);

		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					Stop();
				}),
				FCanExecuteAction::CreateLambda([this]
				{
					return bIsPlaying;
				})
			),
			NAME_None,
			LOCTEXT("Stop_Label", "Stop"),
			LOCTEXT("Stop_ToolTip", "Stop playing the video feeds"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop")
			);
	}
	ToolBarBuilder.EndSection();


	FSlimHorizontalToolBarBuilder RightToolBarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
	RightToolBarBuilder.BeginSection("Options");
	{
		FUIAction OpenSettingsMenuAction;
		OpenSettingsMenuAction.CanExecuteAction = FCanExecuteAction::CreateLambda([this] { return !bIsPlaying; });

		RightToolBarBuilder.AddComboButton(
			OpenSettingsMenuAction,
			FOnGetContent::CreateRaw(this, &SMediaFrameworkVideoInput::CreateSettingsMenu),
			LOCTEXT("Settings_Label", "Settings"),
			LOCTEXT("Settings_ToolTip", "Settings"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Settings")
		);
	}
	RightToolBarBuilder.EndSection();

	return SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	[
		ToolBarBuilder.MakeWidget()
	]

	+SHorizontalBox::Slot()
	.AutoWidth()
	[
		RightToolBarBuilder.MakeWidget()
	];
}


bool SMediaFrameworkVideoInput::CanPlay() const
{
	const UMediaFrameworkVideoInputSettings* UserSettings = MediaFrameworkVideoInputUtils::GetVideoInputSettings();
	bool bCanPlay = (UserSettings->MediaBundles.Num() || UserSettings->MediaSources.Num()) && !bIsPlaying;

	if (bCanPlay)
	{
		//if a lazy load fails we can't play
		for (const TSoftObjectPtr<UMediaBundle>& SoftMediaBundlePtr : UserSettings->MediaBundles)
		{
			bCanPlay = SoftMediaBundlePtr.LoadSynchronous() != nullptr;
			if (!bCanPlay)
			{
				return bCanPlay;
			}
		}
		for (const FMediaFrameworkVideoInputSourceSettings& Media : UserSettings->MediaSources)
		{
			bCanPlay = Media.MediaSource.LoadSynchronous() != nullptr && Media.MediaTexture.LoadSynchronous() != nullptr;
			if (!bCanPlay)
			{
				return bCanPlay;
			}
		}

	}
	return bCanPlay;
}

void SMediaFrameworkVideoInput::Play()
{
	bool bCanPlay = !bIsPlaying;

	if (bCanPlay)
	{
		bCanPlay = CanPlay();
	}

	if (bCanPlay)
	{
		UMediaFrameworkVideoInputSettings* UserSettings = MediaFrameworkVideoInputUtils::GetVideoInputSettings();

		for (TSoftObjectPtr<UMediaBundle>& SoftMediaBundlePtr : UserSettings->MediaBundles)
		{
			UMediaBundle* MediaBundle = SoftMediaBundlePtr.LoadSynchronous();
			if (MediaBundle)
			{
				VideosViewport->DisplayVideoInput(MediaBundle);
			}
		}

		for (const FMediaFrameworkVideoInputSourceSettings& Media : UserSettings->MediaSources)
		{
			UMediaSource* MediaSource = Media.MediaSource.LoadSynchronous();
			UMediaTexture* MediaTexture = Media.MediaTexture.LoadSynchronous();
			if (MediaSource && MediaTexture)
			{
				VideosViewport->DisplayVideoInput(MediaSource, MediaTexture);
			}
		}

		bIsPlaying = true;
	}
}

void SMediaFrameworkVideoInput::Stop()
{
	if (bIsPlaying)
	{
		VideosViewport->Empty();
		bIsPlaying = false;
	}
}

TSharedRef<SWidget> SMediaFrameworkVideoInput::CreateSettingsMenu()
{
	FMenuBuilder SettingsMenuBuilder(true, nullptr);

	{
		SettingsMenuBuilder.AddMenuEntry(
			LOCTEXT("AutoBundleRestart_Label", "Auto re-open Media Bundle"),
			LOCTEXT("AutoBundleRestart_Tooltip", "When a Media Bundle close by error, re-open it if the option is not enabled on the Bundle."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bReopenMediaBundles = !MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bReopenMediaBundles;
					MediaFrameworkVideoInputUtils::GetVideoInputSettings()->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]
				{
					return MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bReopenMediaBundles;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		SettingsMenuBuilder.AddMenuEntry(
			LOCTEXT("AutoSourceRestart_Label", "Auto re-open Media Source"),
			LOCTEXT("AutoSourceRestart_Tooltip", "When a Media Source close by error, re-open it."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bReopenMediaSources = !MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bReopenMediaSources;
					MediaFrameworkVideoInputUtils::GetVideoInputSettings()->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]
				{
					return MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bReopenMediaSources;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	SettingsMenuBuilder.AddMenuSeparator();

	{
		SettingsMenuBuilder.AddMenuEntry(
			LOCTEXT("SplitterOrientation_Label", "Vertical Split"),
			LOCTEXT("SplitterOrientation_Tooltip", "Split the sources vertically or horizontally."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]
				{
					MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bIsVerticalSplitterOrientation = !MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bIsVerticalSplitterOrientation;
					Splitter->SetOrientation(MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bIsVerticalSplitterOrientation ? EOrientation::Orient_Vertical : EOrientation::Orient_Horizontal);
					MediaFrameworkVideoInputUtils::GetVideoInputSettings()->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]
				{
					return MediaFrameworkVideoInputUtils::GetVideoInputSettings()->bIsVerticalSplitterOrientation;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);
	}

	return SettingsMenuBuilder.MakeWidget();
}

void SMediaFrameworkVideoInput::OnAssetsPreDelete(const TArray<UObject*>& Objects)
{
	bool bCheck = false;
	for (UObject* Object : Objects)
	{
		if (Cast<UMediaBundle>(Object) || Cast<UMediaPlayer>(Object) || Cast<UMediaSource>(Object) || Cast<UMediaTexture>(Object) || Cast<UMediaPlayer>(Object))
		{
			bCheck = true;
			break;
		}
	}

	if (bCheck && MediaFrameworkVideoInputUtils::AreSettingsDependentOn(Objects))
	{
		Stop();
	}
}

void SMediaFrameworkVideoInput::OnObjectPreEditChange(UObject* Object, const FEditPropertyChain& PropertyChain)
{
	UMediaFrameworkVideoInputSettings* UserSettings = MediaFrameworkVideoInputUtils::GetVideoInputSettings();
	if (Object == UserSettings)
	{
		Stop();
	}
}

void SMediaFrameworkVideoInput::OnObjectPostEditChange(UObject* Object, FPropertyChangedEvent& InPropertyChangedEvent)
{
	UMediaFrameworkVideoInputSettings* UserSettings = MediaFrameworkVideoInputUtils::GetVideoInputSettings();
	if (UserSettings == Object)
	{
		Object->SaveConfig();
	}
}

#undef LOCTEXT_NAMESPACE
