// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/SCompoundWidget.h"

struct FPropertyChangedEvent;

class FWorkspaceItem;
class IDetailsView;
class UMediaPlayer;
class SSplitter;


namespace MediaFrameworkVideoInputUtils
{
	class SVideoInputDisplayVerticalBox;
}


/*
 * SMediaFrameworkVideoInput is a class that manage the video input tab and it's registration in the level editor
 */
class SMediaFrameworkVideoInput : public SCompoundWidget
{
public:
	static void RegisterNomadTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem);
	static void UnregisterNomadTabSpawner();

public:
	SLATE_BEGIN_ARGS(SMediaFrameworkVideoInput){}
	SLATE_END_ARGS()

	virtual ~SMediaFrameworkVideoInput();

	void Construct(const FArguments& InArgs);

	/*
	 * Tell if the video input tab is currently playing
	 */
	bool IsPlaying() const { return bIsPlaying; }

private:
	TSharedRef<class SWidget> MakeToolBar();

	/*
	 * Determinate if the video input player can play it's video sources
	 */
	bool CanPlay() const;

	/**
	 * If it can, start playing and displaying the video sources
	 */
	void Play();

	/**
	 * Stop playing and displaying the video sources
	 */
	void Stop();

	/**
	 * Create the settings menu
	 */
	TSharedRef<SWidget> CreateSettingsMenu();

	void OnAssetsPreDelete(const TArray<UObject*>& Objects);
	void OnObjectPreEditChange(UObject* Object, const FEditPropertyChain& PropertyChain);
	void OnObjectPostEditChange(UObject* Object, FPropertyChangedEvent& InPropertyChangedEvent);

private:
	TSharedPtr<IDetailsView> DetailView;
	TSharedPtr<SSplitter> Splitter;
	TSharedPtr<MediaFrameworkVideoInputUtils::SVideoInputDisplayVerticalBox> VideosViewport;

	bool bIsPlaying;
	static FDelegateHandle LevelEditorTabManagerChangedHandle;
};
