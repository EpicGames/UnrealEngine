// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoInputTab/SMediaFrameworkVideoInputDisplay.h"
#include "MediaFrameworkUtilitiesEditorModule.h"

#include "EditorFontGlyphs.h"
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "MediaBundle.h"
#include "MediaFrameworkVideoInputSettings.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "VideoInputDisplayUtilities"

/*
 * VideoInputDisplayUtilities
 */
namespace VideoInputDisplayUtilities
{
	double PreviousWarningTime = 0.0;
	const float PaddingTop = 4.f;
	const float VideoPreviewDesiredSizeY = 200.f;

	float CalculateWitdhForDisplay(UMediaTexture* PreviewTextutre)
	{
		if (PreviewTextutre)
		{
			return VideoPreviewDesiredSizeY * PreviewTextutre->GetAspectRatio();
		}
		return VideoPreviewDesiredSizeY;
	}

	TSharedRef<SWidget> ConstructDefaultVideoDisplay()
	{
		return
			SNew(SBox)
			.HeightOverride(VideoPreviewDesiredSizeY)
			.WidthOverride(VideoPreviewDesiredSizeY)
			[
				SNew(SImage)
				.Image(FAppStyle::GetDefaultBrush())
			];
	}
}

/*
 * UMediaFrameworkVideoInputDisplayCallback
 */
void UMediaFrameworkVideoInputDisplayCallback::OnMediaClosed()
{
	TSharedPtr<SMediaFrameworkVideoInputDisplay> OwnerPtr = Owner.Pin();
	if (OwnerPtr.IsValid())
	{
		OwnerPtr->OnMediaClosed();
	}
}

/*
 * SMediaFrameworkVideoInputDisplay
 */
SMediaFrameworkVideoInputDisplay::SMediaFrameworkVideoInputDisplay()
	: Collector(this)
	, Material(nullptr)
	, MediaPlayerCallback(nullptr)
	, MaterialBrush(nullptr)
{ }

SMediaFrameworkVideoInputDisplay::~SMediaFrameworkVideoInputDisplay()
{
	if (MediaPlayerCallback)
	{
		MediaPlayerCallback->MarkAsGarbage();
	}

	if (GEditor && RestartPlayerTimerHandle.IsValid())
	{
		GEditor->GetTimerManager()->ClearTimer(RestartPlayerTimerHandle);
	}
}

void SMediaFrameworkVideoInputDisplay::Construct(const FString& SourceName)
{
	ChildSlot
	[
		SNew(SBorder)
		.Padding(VideoInputDisplayUtilities::PaddingTop)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					ConstructVideoStateDisplay(SourceName)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(EHorizontalAlignment::HAlign_Center)
				[
					ConstructVideoDisplay()
				]
			]
		]
	];
}

void SMediaFrameworkVideoInputDisplay::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(Material);
	InCollector.AddReferencedObject(MediaPlayerCallback);
}

TSharedRef<SWidget> SMediaFrameworkVideoInputDisplay::ConstructVideoStateDisplay(const FString& SourceName)
{
	const FMargin SourceTextPadding(6.f, 2.f, 0.f, 2.f);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
			.Text(this, &SMediaFrameworkVideoInputDisplay::HandleSourceStateText)
			.ColorAndOpacity(this, &SMediaFrameworkVideoInputDisplay::HandleSourceStateColorAndOpacity)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(SourceTextPadding)
		[
			SNew(STextBlock)
			.Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalText")))
			.Text(FText::FromString(SourceName))
		];
}

void SMediaFrameworkVideoInputDisplay::AttachCallback()
{
	UMediaPlayer* MediaPlayer = GetMediaPlayer();
	if (MediaPlayer)
	{
		MediaPlayerCallback = NewObject<UMediaFrameworkVideoInputDisplayCallback>();
		TSharedRef<SMediaFrameworkVideoInputDisplay> SharedRef = StaticCastSharedRef<SMediaFrameworkVideoInputDisplay>(AsShared());
		MediaPlayerCallback->Owner = StaticCastSharedRef<SMediaFrameworkVideoInputDisplay>(AsShared());//TWeakPtr<SMediaFrameworkVideoInputDisplay>(SharedRef);

		MediaPlayer->OnMediaClosed.AddUniqueDynamic(MediaPlayerCallback, &UMediaFrameworkVideoInputDisplayCallback::OnMediaClosed);
	}
}

void SMediaFrameworkVideoInputDisplay::DetachCallback()
{
	UMediaPlayer* MediaPlayer = GetMediaPlayer();
	if (MediaPlayer && MediaPlayerCallback)
	{
		MediaPlayer->OnMediaClosed.RemoveAll(MediaPlayerCallback);
	}
	MediaPlayerCallback = nullptr;
}

void SMediaFrameworkVideoInputDisplay::OnMediaClosed()
{
	UMediaTexture* MediaTexture = GetMediaTexture();
	UMediaPlayer* MediaPlayer = GetMediaPlayer();
	if (MediaPlayer && MediaTexture)
	{
		const EMediaState MediaState = MediaPlayer->GetPlayerFacade()->GetPlayer().IsValid() ? MediaPlayer->GetPlayerFacade()->GetPlayer()->GetControls().GetState() : EMediaState::Closed;
		if (MediaState == EMediaState::Closed || MediaState == EMediaState::Error)
		{

			const double TimeNow = FPlatformTime::Seconds();
			const double TimeBetweenWarningsInSeconds = 3.0;
			if (TimeNow - VideoInputDisplayUtilities::PreviousWarningTime > TimeBetweenWarningsInSeconds)
			{
				FNotificationInfo NotificationInfo(LOCTEXT("MediaOpenFailedError", "A Media Player failed. Check Output Log for details."));
				NotificationInfo.ExpireDuration = 2.0f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
				VideoInputDisplayUtilities::PreviousWarningTime = TimeNow;
			}

			if (MediaPlayer == MediaTexture->GetMediaPlayer())
			{
				UE_LOGF(LogMediaFrameworkUtilitiesEditor, Warning, "The MediaTexture '%ls' doesn't reference the MediaPlayer '%ls' anymore.", *MediaTexture->GetName(), *MediaPlayer->GetName());
			}

			if (GEditor)
			{
				auto RestartPlayerLambda = [this]()
				{
					RestartPlayerTimerHandle.Invalidate();

					UMediaPlayer* MediaPlayer = GetMediaPlayer();
					if (MediaPlayer)
					{
						const EMediaState MediaState = MediaPlayer->GetPlayerFacade()->GetPlayer().IsValid() ? MediaPlayer->GetPlayerFacade()->GetPlayer()->GetControls().GetState() : EMediaState::Closed;
						if (MediaState == EMediaState::Closed || MediaState == EMediaState::Error)
						{
							RestartPlayer();
						}
					}
				};

				const float TimerRate = GetDefault<UMediaFrameworkVideoInputSettings>()->ReopenDelay;
				GEditor->GetTimerManager()->SetTimer(RestartPlayerTimerHandle, RestartPlayerLambda, TimerRate, false);
			}
			else
			{
				RestartPlayer();
			}
		}
	}
}

TSharedRef<SWidget> SMediaFrameworkVideoInputDisplay::ConstructVideoDisplay()
{
	UMediaTexture* MediaTexture = GetMediaTexture();
	if (MediaTexture)
	{
		return
			SNew(SBox)
			.HeightOverride(VideoInputDisplayUtilities::VideoPreviewDesiredSizeY)
			.WidthOverride_Lambda([this]()
			{
				return VideoInputDisplayUtilities::CalculateWitdhForDisplay(GetMediaTexture());
			})
			[
				SNew(SMediaImage, MediaTexture)
			];
	}

	return VideoInputDisplayUtilities::ConstructDefaultVideoDisplay();
}

FSlateColor SMediaFrameworkVideoInputDisplay::HandleSourceStateColorAndOpacity() const
{
	UMediaPlayer* MediaPlayer = GetMediaPlayer();
	if (MediaPlayer)
	{
		if (MediaPlayer->IsPlaying())
		{
			return  FLinearColor::Green;
		}
		else if (MediaPlayer->IsPreparing() || MediaPlayer->IsBuffering() || MediaPlayer->IsConnecting())
		{
			return FLinearColor::Yellow;
		}
		else if (MediaPlayer->HasError())
		{
			return FLinearColor::Red;
		}
	}

	return FLinearColor::Red;
}

FText SMediaFrameworkVideoInputDisplay::HandleSourceStateText() const
{
	UMediaPlayer* MediaPlayer = GetMediaPlayer();
	if (MediaPlayer)
	{
		if (MediaPlayer->IsPlaying())
		{
			return  FEditorFontGlyphs::Play;
		}
		else if(MediaPlayer->IsPaused())
		{
			return  FEditorFontGlyphs::Pause;
		}
		else if (MediaPlayer->IsPreparing() || MediaPlayer->IsBuffering() || MediaPlayer->IsConnecting())
		{
			return FEditorFontGlyphs::Hourglass_O;
		}
		else if (MediaPlayer->HasError())
		{
			return FEditorFontGlyphs::Ban;
		}
	}
	return FEditorFontGlyphs::Ban;
}

/*
 * SMediaFrameworkVideoInputMediaBundleDisplay
 */
SMediaFrameworkVideoInputMediaBundleDisplay::SMediaFrameworkVideoInputMediaBundleDisplay()
	: MediaBundle(nullptr)
	, bDidMediaBundleOpen(false)
{ }

SMediaFrameworkVideoInputMediaBundleDisplay::~SMediaFrameworkVideoInputMediaBundleDisplay()
{
	if (MediaBundle && bDidMediaBundleOpen)
	{
		MediaBundle->CloseMediaSource();
	}
}

void SMediaFrameworkVideoInputMediaBundleDisplay::AddReferencedObjects(FReferenceCollector& InCollector)
{
	Super::AddReferencedObjects(InCollector);

	InCollector.AddReferencedObject(MediaBundle);
}

UMediaPlayer* SMediaFrameworkVideoInputMediaBundleDisplay::GetMediaPlayer() const
{
	return MediaBundle ? MediaBundle->GetMediaPlayer() : nullptr;
}

UMediaTexture* SMediaFrameworkVideoInputMediaBundleDisplay::GetMediaTexture() const
{
	return MediaBundle ? MediaBundle->GetMediaTexture() : nullptr;
}

void SMediaFrameworkVideoInputMediaBundleDisplay::RestartPlayer()
{
	if (MediaBundle && bDidMediaBundleOpen && !MediaBundle->bReopenSourceOnError)
	{
		MediaBundle->CloseMediaSource();
		bDidMediaBundleOpen = MediaBundle->OpenMediaSource();
	}
}

void SMediaFrameworkVideoInputMediaBundleDisplay::Construct(const FArguments& InArgs)
{
	MediaBundle = InArgs._MediaBundle.Get();
	if (MediaBundle)
	{
		if (!MediaBundle->bReopenSourceOnError && GetDefault<UMediaFrameworkVideoInputSettings>()->bReopenMediaBundles)
		{
			AttachCallback();
		}
		bDidMediaBundleOpen = MediaBundle->OpenMediaSource();

		SMediaFrameworkVideoInputDisplay::Construct(MediaBundle->GetName());
	}
}

/*
 * SMediaFrameworkVideoInputMediaSourceDisplay
 */
SMediaFrameworkVideoInputMediaSourceDisplay::SMediaFrameworkVideoInputMediaSourceDisplay()
	: SMediaFrameworkVideoInputDisplay()
	, MediaSource(nullptr)
	, MediaPlayer(nullptr)
	, MediaTexture(nullptr)
{ }

SMediaFrameworkVideoInputMediaSourceDisplay::~SMediaFrameworkVideoInputMediaSourceDisplay()
{
	if (MediaPlayer)
	{
		MediaPlayer->Close();
	}
}

void SMediaFrameworkVideoInputMediaSourceDisplay::AddReferencedObjects(FReferenceCollector& InCollector)
{
	Super::AddReferencedObjects(InCollector);
	InCollector.AddReferencedObject(MediaSource);
	InCollector.AddReferencedObject(MediaPlayer);
	InCollector.AddReferencedObject(MediaTexture);
}

UMediaPlayer* SMediaFrameworkVideoInputMediaSourceDisplay::GetMediaPlayer() const
{
	return MediaPlayer;
}

UMediaTexture* SMediaFrameworkVideoInputMediaSourceDisplay::GetMediaTexture() const
{
	return MediaTexture;
}

void SMediaFrameworkVideoInputMediaSourceDisplay::RestartPlayer()
{
	if (MediaPlayer && MediaTexture && MediaSource)
	{
		MediaPlayer->OpenSource(MediaSource);
		MediaPlayer->Play();
	}
}

void SMediaFrameworkVideoInputMediaSourceDisplay::Construct(const FArguments& InArgs)
{
	if (InArgs._MediaSource.IsValid() && InArgs._MediaTexture.IsValid())
	{
		MediaSource = InArgs._MediaSource.Get();
		check(MediaSource);

		MediaTexture = InArgs._MediaTexture.Get();
		check(MediaTexture);
	}

	bool bOpened = false;

	if (MediaSource && MediaTexture)
	{
		MediaPlayer = MediaTexture->GetMediaPlayer();

		if (MediaPlayer)
		{
			if (MediaPlayer->AffectedByPIEHandling)
			{
				UE_LOGF(LogMediaFrameworkUtilitiesEditor, Warning, "The MediaPlayer '%ls' will be closed when a PIE session starts or stop. You should uncheck 'Affected By PIE Handling' on the MediaPlayer.", *MediaPlayer->GetName());
			}

			if (GetDefault<UMediaFrameworkVideoInputSettings>()->bReopenMediaSources)
			{
				AttachCallback();
			}
			bOpened = MediaPlayer->OpenSource(MediaSource);
			bOpened = bOpened && MediaPlayer->Play();
		}
		else
		{
			UE_LOGF(LogMediaFrameworkUtilitiesEditor, Error, "There is not MediaPlayer associated with the MediaTexture '%ls'.", *MediaTexture->GetName());
		}

		SMediaFrameworkVideoInputDisplay::Construct(MediaSource->GetName());

		if (!bOpened)
		{
			FNotificationInfo NotificationInfo(LOCTEXT("MediaOpenFailedError", "A Media Player failed. Check Output Log for details."));
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

#undef LOCTEXT_NAMESPACE
