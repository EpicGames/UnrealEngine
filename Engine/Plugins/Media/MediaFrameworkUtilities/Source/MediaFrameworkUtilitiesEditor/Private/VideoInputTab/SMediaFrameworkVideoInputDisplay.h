// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Engine/TimerHandle.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

#include "SMediaFrameworkVideoInputDisplay.generated.h"

enum class EMediaEvent;
class SMediaFrameworkVideoInputDisplay;
class UMaterial;
class UMediaBundle;
class UMediaPlayer;
class UMediaSource;
class UMediaTexture;

/**
 * Callback object for SMediaFrameworkVideoInputDisplay
 */
UCLASS(MinimalAPI)
class UMediaFrameworkVideoInputDisplayCallback : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION()
	void OnMediaClosed();

	TWeakPtr<SMediaFrameworkVideoInputDisplay> Owner;
};

/**
 * SMediaFrameworkVideoInputDisplay is a specialize widget for the video input tab. It displays a video feed.
 */
class SMediaFrameworkVideoInputDisplay : public SCompoundWidget
{
protected:
	SMediaFrameworkVideoInputDisplay();
	virtual ~SMediaFrameworkVideoInputDisplay();

	void Construct(const FString& SourceName);

	virtual void AddReferencedObjects(FReferenceCollector& InCollector);
	virtual UMediaPlayer* GetMediaPlayer() const = 0;
	virtual UMediaTexture* GetMediaTexture() const = 0;
	virtual void RestartPlayer() = 0;

	void AttachCallback();
	void DetachCallback();
	void OnMediaClosed();
	friend UMediaFrameworkVideoInputDisplayCallback;

private:
	TSharedRef<SWidget> ConstructVideoDisplay();
	TSharedRef<SWidget> ConstructVideoStateDisplay(const FString& SourceName);

	FSlateColor HandleSourceStateColorAndOpacity() const;
	FText HandleSourceStateText() const;

	class FInternalReferenceCollector : public FGCObject
	{
	public:
		FInternalReferenceCollector(SMediaFrameworkVideoInputDisplay* InObject)
			: Object(InObject)
		{ }

		//~ FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& InCollector) override
		{
			Object->AddReferencedObjects(InCollector);
		}
		virtual FString GetReferencerName() const override
		{
			return TEXT("SMediaFrameworkVideoInputDisplay::FInternalReferenceCollector");
		}

	private:
		SMediaFrameworkVideoInputDisplay* Object;
	};
	friend FInternalReferenceCollector;

	/** Collector to keep the UObject alive. */
	FInternalReferenceCollector Collector;

	/** The material that wraps the video texture for display in an SImage. */
	TObjectPtr<	UMaterial> Material;

	/** Callback object for the MediaPlayer. */
	TObjectPtr<UMediaFrameworkVideoInputDisplayCallback> MediaPlayerCallback;

	/** Timer handle when an error occurred. */
	FTimerHandle RestartPlayerTimerHandle;

	/** The Slate brush that renders the material. */
	TSharedPtr<FSlateBrush> MaterialBrush;
};

/**
 * SMediaFrameworkVideoInputMediaBundleDisplay is a specialize widget for the video input tab. It displays the video feed of a MediaBundle
 */
class SMediaFrameworkVideoInputMediaBundleDisplay : public SMediaFrameworkVideoInputDisplay
{
private:
	using Super = SMediaFrameworkVideoInputDisplay;

public:
	SLATE_BEGIN_ARGS(SMediaFrameworkVideoInputMediaBundleDisplay) {}
	SLATE_ARGUMENT(TWeakObjectPtr<UMediaBundle>, MediaBundle)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	SMediaFrameworkVideoInputMediaBundleDisplay();
	virtual ~SMediaFrameworkVideoInputMediaBundleDisplay();

private:
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual UMediaPlayer* GetMediaPlayer() const override;
	virtual UMediaTexture* GetMediaTexture() const override;
	virtual void RestartPlayer() override;

	TObjectPtr<UMediaBundle> MediaBundle;
	bool bDidMediaBundleOpen;
};

/**
 * SMediaFrameworkVideoInputMediaSourceDisplay is a specialize widget for the video input tab. It displays the video feed of a MediaSource
 */
class SMediaFrameworkVideoInputMediaSourceDisplay : public SMediaFrameworkVideoInputDisplay
{
private:
	using Super = SMediaFrameworkVideoInputDisplay;

public:
	SLATE_BEGIN_ARGS(SMediaFrameworkVideoInputMediaSourceDisplay) {}
	SLATE_ARGUMENT(TWeakObjectPtr<UMediaSource>, MediaSource)
	SLATE_ARGUMENT(TWeakObjectPtr<UMediaTexture>, MediaTexture)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	SMediaFrameworkVideoInputMediaSourceDisplay();
	virtual ~SMediaFrameworkVideoInputMediaSourceDisplay();

private:
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual UMediaPlayer* GetMediaPlayer() const override;
	virtual UMediaTexture* GetMediaTexture() const override;
	virtual void RestartPlayer() override;

	TObjectPtr<UMediaSource> MediaSource;
	TObjectPtr<UMediaPlayer> MediaPlayer;
	TObjectPtr<UMediaTexture> MediaTexture;
};
